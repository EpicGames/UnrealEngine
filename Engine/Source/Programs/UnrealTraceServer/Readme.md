
# Unreal Trace Server

Unreal Trace Server acts as a hub between runtimes that are tracing performance instrumentation and tools like Unreal Insights that consume and present that data for analysis. By default TCP ports 1981 and 1989 are used, where the former receives trace data, and the latter is used by tools to query the server's store.

The server stores new traces to the "store directory". It can also watch for trace files in "watch directories".

The server stores persistent configuration files in:
* `%USERPROFILE%/UnrealEngine/Common/UnrealTrace` on Windows
* `~/UnrealEngine/UnrealTrace` on Linux and MacOS.

## Connecting to the server

When running UnrealTraceServer as a centralized server make sure the following ports are available to clients:

 * Recorder port (default 1981)
 * Store port (default 1989)
 * Dynamic ports (see operating system configuration)

# Building and running

Unreal Trace Server uses [xmake](https://xmake.io) for building and generating solution files. This gives us many benefits:

 * Unified build files across platforms
 * Ability to easily use custom toolchains (like musl)
 * Potentially remove third party source and rely on xmakes package manager
 * Generate IDE files (Visual Studio, XCode, etc)

To build, you first need to install or download xmake to your machine. Follow the [installation guide](https://xmake.io/#/guide/installation) for your host platform. It is also possible to run xmake in a standalone mode (without installing) as long as xmake is in the `PATH` environment variable.

Once installed navigate to the project directory. To build the server run:
```
> xmake
```
The output files are emitted into `build/[platform]/[architecture]`.

### Running
Unreal Trace Server requires a command to run.
* `fork` - Starts a background server, upgrading any existing instance.
* `daemon` - The mode that a background server runs in.
* `kill` - Shuts down a currently running instance.

As long a no other instance of the server is currently running, start the server by: 
```
>  xmake run UnrealTraceServer daemon
```


### IDE files
Optionally IDE files can be generated, for example Visual Studio solution, by using:
```
> xmake project -k vsxmake -m "debug,release"
```

# Making a release

1. Bump `TS_VERSION_MINOR` so the auto-update mechanisms activate when users receive the newer version. 
2. Make sure release mode is active (default) by running `xmake config -m release`.
3. Run the install script `xmake install`. 

The install script will check out the binaries deployed in the Unreal Engine tree. Submit the binaries together with the `Version.h`.

## Building a universal binary on MacOS

In order to build a universal binary for MacOS you need to first build the `x86_64` binary and then the `arm64` binary. These can be combined during
the installation script.

```
> xmake config -a x86_64; xmake
> xmake config -a arm64; xmake 
> xmake install
```
