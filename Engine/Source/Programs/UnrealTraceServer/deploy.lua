-- Copyright Epic Games, Inc. All Rights Reserved.

import("core.project.config")

-- Templates --------------------------------------------------------

local CHANGELIST_DESC_FORMAT = [[
Change: new
Client:$(env P4CLIENT)
User: $(env P4USER)
Type: restricted
Description: 
%s
Files:
%s
]]

local DESCRIPTION_FORMAT = [[
	Version %s of UnrealTraceServer for $(plat)
	
	UnrealTraceServer executable and debug files.
	Version: %s
	Platform: $(plat)
	Architecture: $(arch) 
	Compiled with musl: %s
]]


-- Extract version number from header -------------------------------
function get_version()
	-- Read lines from header
	local lines = io.lines("src/Version.h")
	local protocol_version = nil
	local minor_version = nil
	for line in lines do
		local b,e,version = string.find(line, "TS_VERSION_PROTOCOL%s+(%d+)")
		if version ~= nil then
			protocol_version = version
		end
		local b,e,version = string.find(line, "TS_VERSION_MINOR%s+(%d+)")
		if version ~= nil then
			minor_version = version 
		end
	end
	-- Return nil or the formatted string
	if protocol_version == nil or minor_version == nil then
		return nil
	end
	return format("%s.%s", protocol_version, minor_version)
end


-- Create changelist for list of files ------------------------------
function create_changelist (version, files)
	-- Remap local paths to repo paths
	repo_paths = {}
	for _, file in ipairs(files) do
		result, err = os.iorunv("p4", {"where", file})
		local b,e = string.find(result, "%S+%s")
		if not (b == nil) then
			local repo_path = string.sub(result, b, e)
			table.insert(repo_paths, "\t"..repo_path)
		end
	end
	local file_list = table.concat(repo_paths, "\n")

	-- Create description field
	local description = vformat(DESCRIPTION_FORMAT, version, version, config.get("musl"))

	-- Create changelist description and write to temp file
	local cl_desc = vformat(CHANGELIST_DESC_FORMAT, description, file_list)
	local cl_desc_file = path.join(os.tmpdir(), "uts_cl_desc.txt")
	io.writefile(cl_desc_file, cl_desc)

	-- Create changelist
	result, err = os.iorunv("p4", {"change", "-i"}, {stdin=cl_desc_file})

	b, e, changelist = string.find(result, "Changelist (%d+) created")
	return changelist
end

-- Main entry point -------------------------------------------------
function make_universal_binary(target, archs)
	local input_files = {}
	local build_dir = path.join("build", "macosx")
	local n = 0
	print("Discovering files for universal binary")
	for _, arch in ipairs(archs) do
		print(" Arch: ".. arch)
		local path = path.join(build_dir, arch, "release", target:name())
		print(" Path: "..path.."...")
		if not os.isfile(path) then
			cprint("    ${red}missing${clear}")
			os.exit(1)
		else
			cprint("    ${green}ok${clear}")
			input_files[n] = path
			n = n + 1
		end
	end

	-- Make sure the output directory exists 
	local output_file = path.join(build_dir, "universal", "release", "UnrealTraceServer")
	os.mkdir(path.directory(output_file))

	-- Run the lipo command 
	os.execv("lipo", table.join(input_files, {"-create", "-output", output_file}))

	cprint("Created ${green}"..output_file.."${clear}")

	return output_file
end

-- Main entry point -------------------------------------------------
function main(target)
	-- Map platform names to UE standard
	local ue_platform_names = {windows = "Win64", linux = "Linux",  macosx = "Mac"}
	local ue_platform = ue_platform_names[val("os")]
	-- Relative install dir for now
	local install_dir = path.join('../../../Binaries', ue_platform)
	-- Extract version
	local version = get_version()

	-- Collect files
	local target_dir = target:targetdir()
	local target_bins = {target:filename()}
	if target:is_plat("windows") then 
		-- Also deploy the pdb
		target_bins = table.join(target_bins, {target:basename()..".pdb"}) 
	end
	if target:is_plat("macosx") then
		-- Build universal binaries on Mac
		local univeral_binary = make_universal_binary(target, {"x86_64", "arm64"})
		target_dir = path.directory(univeral_binary)
		target_bins = {path.filename(univeral_binary)}
	end

	-- Deploy each of the files
	cprint("Installing UnrealTraceServer ${bright green}"..version.."${clear} to ${bright green}"..path.absolute(install_dir).."${clear}")
	local deployed_files = {}
	for idx, bin in ipairs(target_bins) do
		local target_bin = path.join(install_dir, bin)
		local source_bin = path.join(target_dir, bin)
		cprint("Deploying file: ${blue}"..source_bin.."${clear}")
		-- Checkout files in Perforce
		os.exec('p4 edit '..target_bin)
		-- On unix like platforms unlink the target before
		-- in case the server is already running
		if target:is_plat("linux", "macosx") and os.exists(target_bin) then
			os.runv("unlink", {target_bin})
		end
		-- Copy output files to Binaries folder
		cprint("Copy ${blue}"..source_bin.."${clear} to ${green}"..target_bin.."${clear}")
		os.cp(source_bin, install_dir)
		table.insert(deployed_files, path.absolute(target_bin))
	end

	-- Create a changelist with standardized commit message
	local changelist = create_changelist(version, deployed_files)
	if changelist ~= nil then
		cprint("Changelist ${bright green}"..changelist.."${clear} was created.")
	end
end
