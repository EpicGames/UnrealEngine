-- Copyright Epic Games, Inc. All Rights Reserved.

import("core.base.task")
import("utils.archive")

--------------------------------------------------------------------------------
local function _exec(cmd, ...)
    local args = {}
    for _, arg in pairs({...}) do
        if arg then
            table.insert(args, arg)
        end
    end

    print("--", cmd, table.unpack(args))
    local ret = os.execv(cmd, args)
    print()
    return ret
end

--------------------------------------------------------------------------------
local function _build(arch, debug, config_args)
    variant = debug and "debug" or "release"
    cprint("----- Building ${blue}"..arch.."${clear} in ${blue}"..variant.."${clear} -----")
    -- Issuing tasks immediately like this fails. Maybe there is some logic
    -- preventing config to run more than once per script?
    -- task.run("config", {clean=true, arch=arch, mode=variant})
    -- task.run("build", {target="UnrealTraceServer", rebuild=true})
    local ret = _exec(
        "xmake",
        "config",
        "--yes",
        "--clean",
        "--mode="..variant,
        "--arch="..arch,
        config_args)
    if ret > 0 then
        raise("Failed to configure xmake")
    end

    ret = _exec("xmake", "clean", "--yes", "UnrealTraceServer")
    if ret > 0 then
        raise("Clean zenserver failed")
    end

    ret = _exec("xmake", "build", "--yes", "UnrealTraceServer")
    if ret > 0 then
        raise("Build zenserver failed")
    end
end

--------------------------------------------------------------------------------
local function _zip(store_only, zip_path, ...)
    -- Here's the rules; if len(...) is 1 and it is a dir then create a zip with
    -- archive paths like this;
    --
    --   glob(foo/bar/**) -> foo/bar/abc, foo/bar/dir/123 -> zip(abc, dir/123)
    --
    -- Otherwise assume ... is file paths and add without leading directories;
    --
    --   foo/abc, bar/123 -> zip(abc, 123)

    zip_path = path.absolute(zip_path)
    os.tryrm(zip_path)

    local inputs = {...}

    local source_dir = nil
    if #inputs == 1 and os.isdir(inputs[1]) then
        source_dir = inputs[1]
    end

    import("detect.tools.find_7z")
    local cmd_7z = find_7z()
    if cmd_7z then
        input_paths = {}
        if source_dir then
            -- Suffixing a directory path with a "/." will have 7z set the path
            -- for archived files relative to that directory.
            input_paths = { path.join(source_dir, ".") }
        else
            for _, input_path in pairs(inputs) do
                -- If there is a "/./" anywhere in file paths then 7z drops all
                -- directory information and just archives the file by name
                input_path = path.relative(input_path, ".")
                if input_path:sub(2,2) ~= ":" then
                    input_path = "./"..input_path
                end
                table.insert(input_paths, input_path)
            end
        end

        compression_level = "-mx1"
        if store_only then
            compression_level = "-mx0"
        end

        local ret = _exec(cmd_7z, "a", compression_level, zip_path, table.unpack(input_paths))
        if ret > 0 then
            raise("Received error from 7z")
        end
        return
    end

    print("7z not found, falling back to zip")

    import("detect.tools.find_zip")
    zip_cmd = find_zip()
    if zip_cmd then
        local input_paths = inputs
        local cwd = os.curdir()
        if source_dir then
            os.cd(source_dir)
            input_paths = { "." }
        end

        compression_level = "-1"
        if store_only then
            compression_level = "-0"
        end

        local strip_leading_path = nil
        if not source_dir then
            strip_leading_path = "--junk-paths"
        end

        local ret = _exec(zip_cmd, "-r", compression_level, strip_leading_path, zip_path, table.unpack(input_paths))
        if ret > 0 then
            raise("Received error from zip")
        end

        os.cd(cwd)
        return
    end
    print("zip not found")

    raise("Unable to find a suitable zip tool")
end

--------------------------------------------------------------------------------
local function main_windows(signidentity)
    import("core.base.option")

    zip_path = "build/UnrealTraceServer-win64.zip"
    config_args = nil

    _build("x64", false, config_args)

    if signidentity == nil or signidentity == "" then
        print("Skipping signing since no signidentity was given")
    else
        program_files_path = os.getenv("PROGRAMFILES(x86)")
        signtool_path = program_files_path .. "/Windows Kits/10/App Certification Kit/signtool.exe"
        if not os.isfile(signtool_path) then
            raise("Failed signing, unable to locate signtool at "..signtool_path)
        end
        local ret = _exec(signtool_path,
            "sign",
            "/sm",
            "/a",
            "/fd",
            "sha256",
            "/n",
            signidentity,
            "/tr",
            "http://timestamp.digicert.com",
            "/td",
            "SHA256",
            "/v",
            "/as",
            "build/windows/x64/release/UnrealTraceServer.exe")
        if ret > 0 then
            raise("Failed signing UnrealTraceServer binary")
        end
    end

    _zip(false,
        zip_path,
        "build/windows/x64/release/UnrealTraceServer.exe",
        "build/windows/x64/release/UnrealTraceServer.pdb")
end

--------------------------------------------------------------------------------
local function main_mac(signidentity)
    -- Build and universalify
    _build("x86_64", false, "--target_minver=12.5")
    _build("arm64", false, "--target_minver=12.5")

    cprint("----- Creating univeral binary -----")
    os.mkdir("build/macosx/universal/release/")
	local ret = _exec(
        "lipo",
        "-create",
        "-output", "build/macosx/universal/release/UnrealTraceServer",
        "build/macosx/x86_64/release/UnrealTraceServer",
        "build/macosx/arm64/release/UnrealTraceServer")
    if ret > 0 then
        raise("Failed creating universal UnrealTraceServer binary")
    end

    ret = _exec(
        "lipo",
        "-create",
        "-output", "build/macosx/universal/release/zen",
        "build/macosx/x86_64/release/UnrealTraceServer",
        "build/macosx/arm64/release/UnrealTraceServer")
    if ret > 0 then
        raise("Failed creating universal UnrealTraceServer binary")
    end

    cprint("----- Sign univeral binary  -----")
    ret = _exec("codesign",
                "-s",
                signidentity,
                "-f",
                "-v",
                "--deep",
                "build/macosx/universal/release/UnrealTraceServer")
    if ret > 0 then
        raise("Failed signing universal UnrealTraceServer binary")
    end

    -- Zip
    _zip(false,
        "build/UnrealTraceServer-macos.zip",
        "build/macosx/universal/release/UnrealTraceServer")
end

--------------------------------------------------------------------------------
local function main_linux()
    _build("x86_64")

    _zip(false,
        "build/UnrealTraceServer-linux.zip",
        "build/linux/x86_64/release/UnrealTraceServer")
end

--------------------------------------------------------------------------------

import("core.base.option")

function main()
    if is_host("windows") then
        signidentity = option.get("codesignidentity")
        return main_windows(signidentity)
    end

    if is_host("mac") then
        signidentity = option.get("codesignidentity")
        if (signidentity == nil or signidentity == '') then
            signidentity = '-'
        end

        return main_mac(signidentity)
    end

    if is_host("linux") then
        return main_linux()
    end

    raise("'xmake bundle' not currently supported on this platform")
end
