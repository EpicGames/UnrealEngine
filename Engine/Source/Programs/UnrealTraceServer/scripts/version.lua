-- Copyright Epic Games, Inc. All Rights Reserved.


-- Extract version number from header -------------------------------
function main()
	-- Read lines from header
	local lines = io.lines("src/Version.h")
	local protocol_version = nil
	local minor_version = nil
	for line in lines do
		local b,e,version = string.find(line, "TS_VERSION_PROTOCOL%s+(%d+)")
		if version ~= nil then
			protocol_version = version
		end
		local b,e,version = string.find(line, "TS_VERSION_MINOR%s+(%d+)")
		if version ~= nil then
			minor_version = version 
		end
	end
	-- Return nil or the formatted string
	if protocol_version == nil or minor_version == nil then
		return nil
	end
	return format("%s.%s", protocol_version, minor_version)
end
