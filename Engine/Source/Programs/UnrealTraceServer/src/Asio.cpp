// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "Asio.h"

#if defined(_MSC_VER)
#	include <malloc.h>
#else
#	include <alloca.h>
#endif

TS_ASIO_INCLUDE_BEGIN
#include "asio/impl/src.hpp"
TS_ASIO_INCLUDE_END

/* vim: set noexpandtab : */
