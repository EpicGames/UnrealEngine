// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if defined(__clang__)
#	if defined(__APPLE__)
#		define TS_ASIO_INCLUDE_BEGIN \
			_Pragma("GCC diagnostic push") \
			_Pragma("GCC diagnostic ignored \"-Wdeprecated-declarations\"")
#		define TS_ASIO_INCLUDE_END \
			_Pragma("GCC diagnostic pop")
#	else
#		define TS_ASIO_INCLUDE_BEGIN
#		define TS_ASIO_INCLUDE_END
#	endif
#else
#	define TS_ASIO_INCLUDE_BEGIN
#	define TS_ASIO_INCLUDE_END
#endif

#define ASIO_SEPARATE_COMPILATION
#define ASIO_STANDALONE
#define ASIO_NO_EXCEPTIONS
#define ASIO_NO_TYPEID
#define ASIO_DISABLE_NOEXCEPT

namespace asio {
namespace detail {

template <typename ExceptionType>
void throw_exception(const ExceptionType& Exception)
{
	/* Intentionally blank */
}

} // namespace detail
} // namespace asio

TS_ASIO_INCLUDE_BEGIN
#include "asio/connect.hpp"
#include "asio/error.hpp"
#include "asio/io_context.hpp"
#include "asio/ip/tcp.hpp"
#include "asio/read.hpp"
#include "asio/write.hpp"
#include "asio/write_at.hpp"

#if TS_USING(TS_PLATFORM_WINDOWS)
#	include "asio/windows/random_access_handle.hpp"
#	include "asio/windows/object_handle.hpp"
#endif
TS_ASIO_INCLUDE_END

#if TS_USING(TS_PLATFORM_WINDOWS)
#	include <winsock2.h>
#	pragma comment(lib, "ws2_32.lib")
#endif

/* vim: set noexpandtab : */
