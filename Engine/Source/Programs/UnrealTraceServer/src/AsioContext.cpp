// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "AsioContext.h"

////////////////////////////////////////////////////////////////////////////////
FAsioContext::FAsioContext(int32 ThreadCount)
{
	if (ThreadCount < 1)
	{
		ThreadCount = 1;
	}

	IoContext = new asio::io_context(ThreadCount);

	ThreadPool.SetNum(ThreadCount);
}

////////////////////////////////////////////////////////////////////////////////
FAsioContext::~FAsioContext()
{
	Wait();
	delete IoContext;
}

////////////////////////////////////////////////////////////////////////////////
void FAsioContext::Start()
{
	if (bRunning)
	{
		return;
	}

	for (std::thread& Thread : ThreadPool)
	{
		std::thread TempThread([this] () { IoContext->run(); });
		Thread = MoveTemp(TempThread);
	}

	bRunning = true;
}

////////////////////////////////////////////////////////////////////////////////
void FAsioContext::Stop(bool bWait)
{
	if (!bRunning)
	{
		return;
	}

	IoContext->stop();

	if (bWait)
	{
		Wait();
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAsioContext::Wait()
{
	for (std::thread& Thread : ThreadPool)
	{
		if (Thread.joinable())
		{
			Thread.join();
		}
	}

	bRunning = false;
}

////////////////////////////////////////////////////////////////////////////////
asio::io_context& FAsioContext::Get()
{
	return *IoContext;
}

/* vim: set noexpandtab : */
