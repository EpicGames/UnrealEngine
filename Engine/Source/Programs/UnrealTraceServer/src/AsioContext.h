// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Asio.h"
#include "Foundation.h"

#include <thread>

////////////////////////////////////////////////////////////////////////////////
class FAsioContext
{
public:
							FAsioContext(int32 ThreadCount);
							~FAsioContext();
	asio::io_context&		Get();
	void					Start();
	void					Stop(bool bWait=false);
	void					Wait();

private:
	asio::io_context*		IoContext;
	TArray<std::thread>		ThreadPool;
	bool					bRunning = false;
};

/* vim: set noexpandtab : */
