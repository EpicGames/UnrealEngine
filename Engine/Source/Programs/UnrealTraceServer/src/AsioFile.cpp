// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "AsioFile.h"
#include "Foundation.h"

#if !TS_USING(TS_PLATFORM_WINDOWS)
#include <sys/file.h>
#include <errno.h>
#endif

////////////////////////////////////////////////////////////////////////////////
FAsioFile::FAsioFile(asio::io_context& IoContext, uintptr_t OsHandle)
: Handle(IoContext, HandleType::native_handle_type(OsHandle))
{
}

////////////////////////////////////////////////////////////////////////////////
asio::io_context& FAsioFile::GetIoContext()
{
	return Handle.get_executor().context();
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioFile::IsOpen() const
{
	return Handle.is_open();
}

////////////////////////////////////////////////////////////////////////////////
void FAsioFile::Close()
{
	Handle.close();
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioFile::HasDataAvailable() const
{
	auto Inner = const_cast<HandleType&>(Handle).native_handle();

	uint64 FileSize = 0;
#if TS_USING(TS_PLATFORM_WINDOWS)
	LARGE_INTEGER Out;
	GetFileSizeEx(Inner, &Out);
	FileSize = Out.QuadPart;
#else
	struct stat Stat;
	fstat(Inner, &Stat);
	FileSize = Stat.st_size;
	static_assert(sizeof(Stat.st_size) >= sizeof(FileSize), "fstat() reports sizes that are too small");
#endif
	return Offset < FileSize;
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioFile::Write(const void* Src, uint32 Size, FAsioIoSink* Sink, uint32 Id)
{
	if (!SetSink(Sink, Id))
	{
		return false;
	}

#if TS_USING(TS_PLATFORM_WINDOWS)
	asio::async_write_at(
		Handle,
		Offset,
#else
	asio::async_write(
		Handle,
#endif
		asio::buffer(Src, Size),
		[this] (const asio::error_code& ErrorCode, size_t BytesWritten)
		{
			Offset += BytesWritten;
			return OnIoComplete(ErrorCode, uint32(BytesWritten));
		}
	);

	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioFile::Read(void* Dest, uint32 Size, FAsioIoSink* Sink, uint32 Id)
{
	return ReadSome(Dest, Size, Sink, Id);
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioFile::ReadSome(void* Dest, uint32 DestSize, FAsioIoSink* Sink, uint32 Id)
{
	if (!SetSink(Sink, Id))
	{
		return false;
	}

#if TS_USING(TS_PLATFORM_WINDOWS)
	Handle.async_read_some_at(
		Offset,
#else
	Handle.async_read_some(
#endif
		asio::buffer(Dest, DestSize),
		[this] (const asio::error_code& ErrorCode, size_t BytesRead)
		{
			Offset += BytesRead;
			return OnIoComplete(ErrorCode, uint32(BytesRead));
		}
	);

	return true;
}

////////////////////////////////////////////////////////////////////////////////
FAsioWriteable* FAsioFile::WriteFile(asio::io_context& IoContext, const FPath& Path)
{
#if TS_USING(TS_PLATFORM_WINDOWS)
	HANDLE Handle = CreateFileW(Path.c_str(), GENERIC_WRITE, FILE_SHARE_READ, nullptr,
		CREATE_ALWAYS, FILE_FLAG_OVERLAPPED|FILE_ATTRIBUTE_NORMAL, nullptr);
	if (Handle == INVALID_HANDLE_VALUE)
	{
		return nullptr;
	}
	return new FAsioFile(IoContext, uintptr_t(Handle));
#else
	int File = open(Path.c_str(), O_WRONLY|O_CREAT, 0666);
	if (!File)
	{
		return nullptr;
	}
	return new FAsioFile(IoContext, uintptr_t(File));
#endif
}

////////////////////////////////////////////////////////////////////////////////
FAsioReadable* FAsioFile::ReadFile(asio::io_context& IoContext, const FPath& Path)
{
#if TS_USING(TS_PLATFORM_WINDOWS)
	HANDLE Handle = CreateFileW(Path.c_str(), GENERIC_READ, FILE_SHARE_READ | FILE_SHARE_WRITE,
		nullptr, OPEN_EXISTING, FILE_FLAG_OVERLAPPED, nullptr);
	if (Handle == INVALID_HANDLE_VALUE)
	{
		return nullptr;
	}
	return new FAsioFile(IoContext, uintptr_t(Handle));
#else
	int File = open(Path.c_str(), O_RDONLY, 0444);
	if (!File)
	{
		return nullptr;
	}
	return new FAsioFile(IoContext, uintptr_t(File));
#endif
}

/* vim: set noexpandtab : */
