// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Asio.h"

#include "AsioIoable.h"
#include "Foundation.h"
#if !TS_USING(TS_PLATFORM_WINDOWS)
#include "asio/posix/stream_descriptor.hpp"
#endif

////////////////////////////////////////////////////////////////////////////////
class FAsioFile
	: public FAsioReadable
	, public FAsioWriteable
{
public:
										FAsioFile(asio::io_context& IoContext, uintptr_t OsHandle);
	asio::io_context&					GetIoContext();
	static FAsioWriteable*				WriteFile(asio::io_context& IoContext, const FPath& Path);
	static FAsioReadable*				ReadFile(asio::io_context& IoContext, const FPath& Path);
	virtual bool						IsOpen() const override;
	virtual void						Close() override;
	virtual bool						HasDataAvailable() const override;
	virtual bool						Write(const void* Src, uint32 Size, FAsioIoSink* Sink, uint32 Id) override;
	virtual bool						Read(void* Dest, uint32 Size, FAsioIoSink* Sink, uint32 Id) override;
	virtual bool						ReadSome(void* Dest, uint32 BufferSize, FAsioIoSink* Sink, uint32 Id) override;

private:
	using HandleType =
#if TS_USING(TS_PLATFORM_WINDOWS)
		asio::windows::random_access_handle;
#else
		asio::posix::stream_descriptor;
#endif

	HandleType							Handle;
	uint64								Offset = 0;
};

/* vim: set noexpandtab : */
