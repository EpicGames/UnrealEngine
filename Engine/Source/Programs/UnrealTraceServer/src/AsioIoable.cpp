// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "AsioIoable.h"
#include "Logging.h"

////////////////////////////////////////////////////////////////////////////////
bool FAsioIoable::SetSink(FAsioIoSink* Ptr, uint32 Id)
{
	if (SinkPtr != nullptr)
	{
		return false;
	}

	SinkPtr = Ptr;
	SinkId = Id;
	return true;
}

////////////////////////////////////////////////////////////////////////////////
void FAsioIoable::OnIoComplete(const asio::error_code& ErrorCode, int32 Size)
{
	if (SinkPtr == nullptr)
	{
		return;
	}

#if TS_USING(TS_BUILD_DEBUG) && 1
	if (ErrorCode && ErrorCode != asio::error::eof)
	{
		switch (ErrorCode.value())
		{
		case asio::error::eof: 
			break;
		case asio::error::connection_aborted:
		case asio::error::connection_reset:
			TS_LOG("Connection closed (object %p)", this);
			break;
		default:
			TS_LOG("IO error (object %p): (Code %u) %s", this, ErrorCode, ErrorCode.message().c_str());
		}
	}
#endif

	if (ErrorCode)
	{
		Size = 0 - ErrorCode.value();
	}

	FAsioIoSink* Ptr = SinkPtr;
	SinkPtr = nullptr;
	Ptr->OnIoComplete(SinkId, Size);
}

/* vim: set noexpandtab : */
