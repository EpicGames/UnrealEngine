// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Asio.h"

////////////////////////////////////////////////////////////////////////////////
class FAsioIoSink
{
public:
	virtual void	OnIoComplete(uint32 Id, int32 Size) = 0;
};

////////////////////////////////////////////////////////////////////////////////
class FAsioIoable
{
public:
	virtual			~FAsioIoable() = default;
	virtual bool	IsOpen() const = 0;
	virtual void	Close() = 0;

protected:
	bool			SetSink(FAsioIoSink* Ptr, uint32 Id);
	void			OnIoComplete(const asio::error_code& ErrorCode, int32 Size);

private:
	FAsioIoSink*	SinkPtr = nullptr;
	uint32			SinkId;
};

////////////////////////////////////////////////////////////////////////////////
class FAsioReadable
	: public virtual FAsioIoable
{
public:
	virtual bool	HasDataAvailable() const = 0;
	virtual bool	Read(void* Dest, uint32 Size, FAsioIoSink* Sink, uint32 Id) = 0;
	virtual bool	ReadSome(void* Dest, uint32 DestSize, FAsioIoSink* Sink, uint32 Id) = 0;
};



////////////////////////////////////////////////////////////////////////////////
class FAsioWriteable
	: public virtual FAsioIoable
{
public:
	virtual bool	Write(const void* Src, uint32 Size, FAsioIoSink* Sink, uint32 Id) = 0;
};

/* vim: set noexpandtab : */
