// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "AsioSocket.h"
#include "Foundation.h"
#include "Logging.h"

#if TS_USING(TS_PLATFORM_WINDOWS)
#	include <Mstcpip.h>
#endif

////////////////////////////////////////////////////////////////////////////////
FAsioSocket::FAsioSocket(asio::ip::tcp::socket& InSocket, const FKeepAliveConfig& Config)
: Socket(MoveTemp(InSocket))
{
	if (Config.Enabled)
	{
		int Error(0), ErrorCode(0);

#if TS_USING(TS_PLATFORM_WINDOWS)
		tcp_keepalive KeepAlive =
		{
			Config.Enabled,		// on
			Config.TimeoutSec * 1000,	// timeout_ms
			Config.IntervalSec * 1000,	// interval_ms
		};

		DWORD BytesReturned;
		Error = WSAIoctl(
			Socket.native_handle(),
			SIO_KEEPALIVE_VALS,
			&KeepAlive, sizeof(KeepAlive),
			nullptr, 0,
			&BytesReturned,
			nullptr,
			nullptr
		);
		if (Error)
		{
			ErrorCode = WSAGetLastError();
		}

#elif TS_USING(TS_PLATFORM_MAC)
		int Enabled = Config.Enabled, Secs = Config.TimeoutSec;

		Error |= setsockopt(Socket.native_handle(), SOL_SOCKET, SO_KEEPALIVE, &Enabled, sizeof(Enabled));
		Error |= setsockopt(Socket.native_handle(), IPPROTO_TCP, TCP_KEEPALIVE, &Secs, sizeof(Secs));

		if (Error)
		{
			ErrorCode = errno;
		}
#elif TS_USING(TS_PLATFORM_LINUX)
		int Enabled = Config.Enabled, KeepCnt = 5, KeepIdle = Config.TimeoutSec, KeepIntvl = Config.IntervalSec;

		Error |= setsockopt(Socket.native_handle(), SOL_SOCKET, SO_KEEPALIVE, &Enabled, sizeof Enabled);
		Error |= setsockopt(Socket.native_handle(), IPPROTO_TCP, TCP_KEEPCNT, &KeepCnt, sizeof(int));
		Error |= setsockopt(Socket.native_handle(), IPPROTO_TCP, TCP_KEEPIDLE, &KeepIdle, sizeof(int));
		Error |= setsockopt(Socket.native_handle(), IPPROTO_TCP, TCP_KEEPINTVL, &KeepIntvl, sizeof(int));

		if (Error)
		{
			ErrorCode = errno;
		}
#endif

		if (Error)
		{
			TS_LOG("Failed to set keep alive for recording socket. Error: %d", ErrorCode);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
FAsioSocket::~FAsioSocket()
{
	check(!IsOpen());
}

////////////////////////////////////////////////////////////////////////////////
asio::io_context& FAsioSocket::GetIoContext()
{
	return Socket.get_executor().context();
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioSocket::IsOpen() const
{
	return Socket.is_open();
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioSocket::HasDataAvailable() const
{
	return (Socket.available() > 0);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FAsioSocket::GetRemoteAddress() const
{
	if (!IsOpen())
	{
		return 0;
	}

	const asio::ip::tcp::endpoint Endpoint = Socket.remote_endpoint();
	asio::ip::address Address = Endpoint.address();
	return Address.is_v4() ? Address.to_v4().to_uint() : 0;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FAsioSocket::GetRemotePort() const
{
	if (!IsOpen())
	{
		return 0;
	}

	const asio::ip::tcp::endpoint Endpoint = Socket.remote_endpoint();
	return Endpoint.port();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FAsioSocket::GetLocalPort() const
{
	if (!IsOpen())
	{
		return 0;
	}

	const asio::ip::tcp::endpoint Endpoint = Socket.local_endpoint();
	return Endpoint.port();
}

////////////////////////////////////////////////////////////////////////////////
void FAsioSocket::Close()
{
	Socket.close();
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioSocket::IsLocalConnection() const
{
	return Socket.local_endpoint().address() == Socket.remote_endpoint().address();
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioSocket::Read(void* Dest, uint32 Size, FAsioIoSink* Sink, uint32 Id)
{
	if (!SetSink(Sink, Id))
	{
		return false;
	}

	asio::async_read(
		Socket,
		asio::buffer(Dest, Size),
		[this] (const asio::error_code& ErrorCode, size_t BytesReceived)
		{
			OnIoComplete(ErrorCode, uint32(BytesReceived));
		}
	);

	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioSocket::ReadSome(void* Dest, uint32 BufferSize, FAsioIoSink* Sink, uint32 Id)
{
	if (!SetSink(Sink, Id))
	{
		return false;
	}

	Socket.async_receive(
		asio::buffer(Dest, BufferSize),
		[this] (const asio::error_code& ErrorCode, size_t BytesReceived)
		{
			return OnIoComplete(ErrorCode, uint32(BytesReceived));
		}
	);

	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioSocket::Write(const void* Src, uint32 Size, FAsioIoSink* Sink, uint32 Id)
{
	if (!SetSink(Sink, Id))
	{
		return false;
	}

	asio::async_write(
		Socket,
		asio::buffer(Src, Size),
		[this] (const asio::error_code& ErrorCode, size_t BytesSent)
		{
			return OnIoComplete(ErrorCode, uint32(BytesSent));
		}
	);

	return true;
}

/* vim: set noexpandtab : */
