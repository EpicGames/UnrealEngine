// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Asio.h"
#include "AsioIoable.h"

////////////////////////////////////////////////////////////////////////////////
class FAsioSocket
	: public FAsioReadable
	, public FAsioWriteable
{
public:

	struct FKeepAliveConfig
	{
		bool Enabled = false;
		uint32 TimeoutSec = 0;
		uint32 IntervalSec = 0;
	};
	static constexpr FKeepAliveConfig DefaultKeepAlive = FKeepAliveConfig {false, 0, 0};

							FAsioSocket(asio::ip::tcp::socket& InSocket, const FKeepAliveConfig& KeepAlive = DefaultKeepAlive);
	virtual					~FAsioSocket();
	asio::io_context&		GetIoContext();
	virtual bool			IsOpen() const override;
	virtual void			Close() override;
	virtual bool			HasDataAvailable() const override;
	bool					IsLocalConnection() const;
	uint32					GetRemoteAddress() const;
	uint32					GetRemotePort() const;
	uint32					GetLocalPort() const;
	virtual bool			Read(void* Dest, uint32 Size, FAsioIoSink* Sink, uint32 Id) override;
	virtual bool			ReadSome(void* Dest, uint32 DestSize, FAsioIoSink* Sink, uint32 Id) override;
	virtual bool			Write(const void* Src, uint32 Size, FAsioIoSink* Sink, uint32 Id) override;

private:
	asio::ip::tcp::socket	Socket;
};

/* vim: set noexpandtab : */
