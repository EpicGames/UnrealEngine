// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "AsioTcpServer.h"
#include "Foundation.h"
#include "Logging.h"

////////////////////////////////////////////////////////////////////////////////
FAsioTcpServer::FAsioTcpServer(asio::io_context& IoContext)
: Acceptor(IoContext)
{
}

////////////////////////////////////////////////////////////////////////////////
FAsioTcpServer::~FAsioTcpServer()
{
	check(!IsOpen());
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioTcpServer::IsOpen() const
{
	return Acceptor.is_open();
}

////////////////////////////////////////////////////////////////////////////////
asio::io_context& FAsioTcpServer::GetIoContext()
{
	return Acceptor.get_executor().context();
}

////////////////////////////////////////////////////////////////////////////////
void FAsioTcpServer::Close()
{
	Acceptor.close();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FAsioTcpServer::GetPort() const
{
	using asio::ip::tcp;

	if (!Acceptor.is_open())
	{
		return 0;
	}

	asio::error_code ErrorCode;
	tcp::endpoint Endpoint = Acceptor.local_endpoint(ErrorCode);
	if (ErrorCode)
	{
		return 0;
	}

	return Endpoint.port();
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioTcpServer::StartServer(uint32 Port, uint32 Backlog)
{
	if (Acceptor.is_open())
	{
		return false;
	}

	using asio::ip::tcp;

	tcp::acceptor TempAcceptor(GetIoContext());

#if TS_USING(TS_PLATFORM_WINDOWS)
	DWORD Flags = WSA_FLAG_NO_HANDLE_INHERIT|WSA_FLAG_OVERLAPPED;
	SOCKET Socket = WSASocketW(AF_INET, SOCK_STREAM, IPPROTO_TCP, nullptr, 0, Flags);
	TempAcceptor.assign(tcp::v4(), Socket);
#else
	TempAcceptor.open(tcp::v4());
#endif

	// Set the reuse address option (SO_REUSEADDR on *nix) otherwise
	// when stopping the server the connection is in timed wait prevent
	// rebinding to that address.
	asio::socket_base::reuse_address reuse_option(true);
	TempAcceptor.set_option(reuse_option);

	tcp::endpoint Endpoint(tcp::v4(), uint16(Port));
	TempAcceptor.bind(Endpoint);
	TempAcceptor.listen(Backlog);

	tcp::endpoint LocalEndpoint = TempAcceptor.local_endpoint();
	if (LocalEndpoint.port() == 0)
	{
		return false;
	}

	Acceptor = MoveTemp(TempAcceptor);
	AsyncAccept();
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioTcpServer::StopServer()
{
	if (!Acceptor.is_open())
	{
		return false;
	}

	Acceptor.close();
	return true;
}

////////////////////////////////////////////////////////////////////////////////
void FAsioTcpServer::AsyncAccept()
{
	using asio::ip::tcp;

	Acceptor.async_accept([this] (
		const asio::error_code& ErrorCode,
		tcp::socket&& Socket)
	{
#if TS_USING(TS_PLATFORM_WINDOWS)
		// This is a best effort and likely not to work because WFP filters (e.g.
		// antimalware or firewalls) can redirect socket handles.
		SetHandleInformation(HANDLE(SOCKET(Socket.native_handle())), HANDLE_FLAG_INHERIT, 0);
#endif

		if (!ErrorCode && OnAccept(Socket))
		{
			AsyncAccept();
			return;
		}
		
		if (ErrorCode == asio::error::operation_aborted)
		{
			TS_LOG("Listening cancelled, closing port...");
		}
		else if (ErrorCode)
		{
			TS_LOG("Accept failed with error %u, closing port...", ErrorCode);
		}

		Close();
	});
}

/* vim: set noexpandtab : */
