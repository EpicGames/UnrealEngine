// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Asio.h"

////////////////////////////////////////////////////////////////////////////////
class FAsioTcpServer
{
public:
							FAsioTcpServer(asio::io_context& IoContext);
	virtual					~FAsioTcpServer();
	asio::io_context&		GetIoContext();
	bool					IsOpen() const;
	void					Close();
	virtual bool			OnAccept(asio::ip::tcp::socket& Socket) = 0;
	uint32					GetPort() const;
	bool					StartServer(uint32 Port=0, uint32 Backlog=1);
	bool					StopServer();

private:
	void					AsyncAccept();
	asio::ip::tcp::acceptor	Acceptor;
};

/* vim: set noexpandtab : */
