// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "AsioTickable.h"
#include "Foundation.h"

#include <chrono>

////////////////////////////////////////////////////////////////////////////////
FAsioTickable::FAsioTickable(asio::io_context& IoContext)
: Timer(IoContext)
{
}

////////////////////////////////////////////////////////////////////////////////
FAsioTickable::~FAsioTickable()
{
	check(!IsActive());
}

////////////////////////////////////////////////////////////////////////////////
asio::io_context& FAsioTickable::GetIoContext()
{
	return Timer.get_executor().context();
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioTickable::IsActive() const
{
	return MillisecondRate != 0;
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioTickable::StartTick(uint32 InMillisecondRate)
{
	if (MillisecondRate)
	{
		return false;
	}

	MillisecondRate = InMillisecondRate;
	if (MillisecondRate)
	{
		AsyncTick();
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioTickable::StopTick()
{
	MillisecondRate = 0;
	Timer.cancel();
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FAsioTickable::TickOnce(uint32 InMillisecondRate)
{
	if (MillisecondRate)
	{
		return false;
	}

	MillisecondRate = InMillisecondRate;
	if (MillisecondRate)
	{
		AsyncTick();
		MillisecondRate = 0;
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
void FAsioTickable::AsyncTick()
{
	auto StdTime = std::chrono::milliseconds(MillisecondRate);
	Timer.expires_after(StdTime);

	Timer.async_wait([this] (const asio::error_code& ErrorCode)
	{
		if (ErrorCode)
		{
			return;
		}

		OnTick();

		if (MillisecondRate)
		{
			AsyncTick();
		}
	});
}

/* vim: set noexpandtab : */
