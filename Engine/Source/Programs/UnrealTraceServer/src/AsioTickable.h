// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Asio.h"

////////////////////////////////////////////////////////////////////////////////
class FAsioTickable
{
public:
						FAsioTickable(asio::io_context& IoContext);
	virtual				~FAsioTickable();
	asio::io_context&	GetIoContext();
	bool				IsActive() const;
	bool				StartTick(uint32 MillisecondRate);
	bool				StopTick();
	bool				TickOnce(uint32 MillisecondRate);
	virtual void		OnTick() = 0;

private:
	void				AsyncTick();
	asio::steady_timer	Timer;
	uint32				MillisecondRate = 0;
};

/* vim: set noexpandtab : */
