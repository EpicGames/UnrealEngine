// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "Cbor.h"
#include "Utils.h"

// {{{1 misc -------------------------------------------------------------------

enum ECborMajorType
{
	Integer			= 0 << 5,
	NegativeInteger	= 1 << 5,
	ByteString		= 2 << 5,
	TextString		= 3 << 5,
	Array			= 4 << 5,
	Map				= 5 << 5,
	Tag				= 6 << 5,
	Float_Simple	= 7 << 5,
};



// {{{1 context ----------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
void FCborContext::Reset()
{
	NextCursor = nullptr;
}

////////////////////////////////////////////////////////////////////////////////
ECborType FCborContext::GetType() const
{
	return Type;
}

////////////////////////////////////////////////////////////////////////////////
int64 FCborContext::GetLength() const
{
	return Param;
}

////////////////////////////////////////////////////////////////////////////////
FStringView	FCborContext::AsString() const
{
	if (GetType() != ECborType::String)
	{
		return FStringView();
	}

    return FStringView((const char*)(NextCursor - Param), std::size_t(Param));
}

////////////////////////////////////////////////////////////////////////////////
int64 FCborContext::AsInteger() const
{
    return Param;
}

////////////////////////////////////////////////////////////////////////////////
void FCborContext::SetError()
{
	Type = ECborType::Error;
	Param = -1;
	NextCursor = nullptr;
}



// {{{1 writer -----------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
FCborWriter::FCborWriter(FInlineBuffer& InBuffer)
: Buffer(InBuffer)
{
}

////////////////////////////////////////////////////////////////////////////////
void FCborWriter::OpenMap(int32 ItemCount)
{
	if (ItemCount != -1)
	{
		WriteParam(ECborMajorType::Map, ItemCount);
		return;
	}

	uint8 Initial = ECborMajorType::Map | 31;
	Buffer.Append(&Initial, sizeof(Initial));
}

////////////////////////////////////////////////////////////////////////////////
void FCborWriter::OpenArray(int32 ItemCount)
{
	if (ItemCount != -1)
	{
		WriteParam(ECborMajorType::Array, ItemCount);
		return;
	}

	uint8 Initial = ECborMajorType::Array | 31;
	Buffer.Append(&Initial, sizeof(Initial));
}

////////////////////////////////////////////////////////////////////////////////
void FCborWriter::Close()
{
	uint8 Initial = ECborMajorType::Float_Simple | 31;
	Buffer.Append(&Initial, sizeof(Initial));
}

////////////////////////////////////////////////////////////////////////////////
void FCborWriter::WriteString(const char* Value, int32 Length)
{
	Length = (Length < 0) ? int32(strlen(Value)) : Length;

	WriteParam(ECborMajorType::TextString, Length);
	Buffer.Append(Value, Length);
}

////////////////////////////////////////////////////////////////////////////////
void FCborWriter::WriteInteger(int64 Value)
{
	uint32 Negative = (Value < 0);
	if (Negative)
	{
		Value = ~Value;
	}

	uint32 MajorType = Negative ? ECborMajorType::NegativeInteger : ECborMajorType::Integer;
	WriteParam(MajorType, Value);
}

////////////////////////////////////////////////////////////////////////////////
void FCborWriter::WriteParam(uint32 MajorType, uint64 Value)
{
	if (Value <= 23)
	{
		uint8 Initial = MajorType | uint8(Value);
		Buffer.Append(&Initial, 1);
		return;
	}

	uint32 BytesPow2 = 3;
	BytesPow2 -= int32(Value <= 0xffff'ffffu);
	BytesPow2 -= int32(Value <= 0x0000'ffffu);
	BytesPow2 -= int32(Value <= 0x0000'00ffu);

	Value = ((Value & 0xffff'ffff'0000'0000ull) >> 32) | ((Value & 0x0000'0000'ffff'ffffull) << 32);
	Value = ((Value & 0xffff'0000'ffff'0000ull) >> 16) | ((Value & 0x0000'ffff'0000'ffffull) << 16);
	Value = ((Value & 0xff00'ff00'ff00'ff00ull) >>  8) | ((Value & 0x00ff'00ff'00ff'00ffull) <<  8);

	Value >>= 64 - (8 << BytesPow2);

	struct {
		uint8	Initial;
		uint8	Payload[sizeof(Value)];
	} Packet = {
		uint8(MajorType | (24 + BytesPow2)),
	};

	memcpy(Packet.Payload, &Value, sizeof(Value));
	Buffer.Append(&Packet, 1 + (1 << BytesPow2));
}



// {{{1 reader -----------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
FCborReader::FCborReader(const uint8* Data, uint32 DataSize)
: Base(Data)
, DataEnd(Data + DataSize)
{
}

////////////////////////////////////////////////////////////////////////////////
int64 FCborReader::ReadParam(const uint8*& Cursor)
{
	uint32 Additional = *Cursor & 0b0001'1111;
	++Cursor;

	// [0-23] literal values
	if (uint32(23 - Additional) <= 23)
	{
		return Additional;
	}

	// [24-27] value in subsequent bytes
	else if (uint32(27 - Additional) <= 27)
	{
		uint32 Bytes = 1 << (Additional - 24);
		if (Cursor + Bytes > DataEnd)
		{
			return -2;
		}

		int64 Value = 0;
		do
		{
			Value <<= 8;
			Value |= *Cursor;
			++Cursor;
		}
		while (--Bytes);

		return Value;
	}

	// 31: indeterminate length
	else if (Additional == 31)
	{
		return -1;
	}

	// ...something went wrong
	return -2;
}

////////////////////////////////////////////////////////////////////////////////
bool FCborReader::ReadContainer(FCborContext& Context, const uint8* Cursor)
{
	bool bIsMap = (*Cursor >= ECborMajorType::Map);

	int64 Param = ReadParam(Cursor);
	if (Param <= -2)
	{
		Context.SetError();
		return false;
	}

	Context.Type = bIsMap ? ECborType::Map : ECborType::Array;
	Context.NextCursor = Cursor;
	Context.Param = Param;
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FCborReader::ReadInteger(FCborContext& Context, const uint8* Cursor)
{
	bool bIsNegative = (*Cursor >= ECborMajorType::NegativeInteger);

	int64 Param = ReadParam(Cursor);
	if (Param <= -1)
	{
		Context.SetError();
		return false;
	}

	Context.Type = ECborType::Integer;
	Context.NextCursor = Cursor;
	Context.Param = bIsNegative ? ~Param : Param;
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FCborReader::ReadString(FCborContext& Context, const uint8* Cursor)
{
	int64 Param = ReadParam(Cursor);
	if (Param <= -2)
	{
		Context.SetError();
		return false;
	}
	else if (Param >= 0)
	{
		if ((Cursor + Param) > DataEnd)
		{
			Context.SetError();
			return false;
		}

		Cursor += Param;
	}

	Context.Type = ECborType::String;
	Context.NextCursor = Cursor;
	Context.Param = Param;
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FCborReader::ReadNext(FCborContext& Context)
{
	const uint8* Cursor = (Context.NextCursor == nullptr) ? Base : Context.NextCursor;
	if (Cursor >= DataEnd)
	{
		Context.Type = ECborType::Eof;
		Context.Param = -1;
		return false;
	}

	int64 Param;
	switch (*Cursor & 0b1110'0000)
	{
	case ECborMajorType::Integer:
	case ECborMajorType::NegativeInteger:
		return ReadInteger(Context, Cursor);

	case ECborMajorType::ByteString:
	case ECborMajorType::TextString:
		return ReadString(Context, Cursor);

	case ECborMajorType::Array:
	case ECborMajorType::Map:
		return ReadContainer(Context, Cursor);
	
	case ECborMajorType::Tag:
		/* unsupported */
		break;

	case ECborMajorType::Float_Simple:
		Param = *Cursor & 0b0001'1111;
		++Cursor;

		switch (Param)
		{
		case 24:
			if (Cursor >= DataEnd)
			{
				Context.SetError();
				return false;
			}

			switch (*Cursor)
			{
			case 20: // false
			case 21: // true
			case 22: // null
			case 23: // undefined
				break;
			}
			break;

		case 25: // half
		case 26: // float
		case 27: // double
			/* if (Cursor + float_size > DataEnd) "basil!" / Context.SetError() */
			break;

		case 31:
			Context.Type = ECborType::End;
			Context.NextCursor = Cursor;
			Context.Param = -1;
			return false; // "false" to facilitate ReadNext() in loops
		}
		// 31 = break
		break;
	}

	return false;
}

