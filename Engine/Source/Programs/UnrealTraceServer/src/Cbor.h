// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*
 * A note to future visitors; the CBOR implementation below is not complete. Just
 * enough to satisfy requirements has been implemented
 */

#include "Foundation.h"

class FInlineBuffer;

////////////////////////////////////////////////////////////////////////////////
enum class ECborType
{
	Error = -1,
	Eof,
	Integer,
	//Boolean,
	//Float,
	String,
	Array,
	Map,
	End, // ..of array/map
	//Null,
	//Undefined,
	_Count,
};

////////////////////////////////////////////////////////////////////////////////
class FCborContext
{
	friend class FCborReader;

public:
	void			Reset();
	ECborType		GetType() const;
	int64			GetLength() const;
	FStringView		AsString() const;
	int64			AsInteger() const;

private:
	void			SetError();
	const uint8*	NextCursor = nullptr;
	int64			Param;
	ECborType		Type;
};

////////////////////////////////////////////////////////////////////////////////
class FCborWriter
{
public:
					FCborWriter(FInlineBuffer& InBuffer);
	void			OpenMap(int32 ItemCount=-1);
	void			OpenArray(int32 ItemCount=-1);
	void			Close();
	void			WriteString(const char* Value, int32 Length=-1);
	void			WriteInteger(int64 Value);

private:
	void			WriteParam(uint32 MajorType, uint64 Value);
	FInlineBuffer&	Buffer;
};

////////////////////////////////////////////////////////////////////////////////
class FCborReader
{
public:
					FCborReader(const uint8* Data, uint32 DataSize);
	bool			ReadNext(FCborContext& Context);

private:
	int64			ReadParam(const uint8*& Cursor);
	bool			ReadContainer(FCborContext& Context, const uint8* Cursor);
	bool			ReadInteger(FCborContext& Context, const uint8* Cursor);
	bool			ReadString(FCborContext& Context, const uint8* Cursor);
	const uint8*	Base;
	const uint8*	DataEnd;
};

/* vim: set noexpandtab : */
