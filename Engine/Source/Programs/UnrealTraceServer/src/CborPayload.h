// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cbor.h"
#include "Foundation.h"
#include "Utils.h"

////////////////////////////////////////////////////////////////////////////////
struct FPayload
{
	const uint8*	Data;
	uint32			Size;
};



////////////////////////////////////////////////////////////////////////////////
enum class EStatusCode
{
	Unknown				= 0,
	Success				= 200,
	BadRequest			= 400,
	Forbidden			= 403,
	NotFound			= 404,
	InternalError		= 500,
};



////////////////////////////////////////////////////////////////////////////////
template <int Size=128>
class TPayloadBuilder
{
public:
							TPayloadBuilder(EStatusCode StatusCode);
	template <int N>		TPayloadBuilder(const char (&Path)[N]);
	template <int N> void	AddInteger(const char (&Name)[N], int64 Value);
	template <int N> void	AddString(const char (&Name)[N], const char* Value, int32 Length=-1);
	template <int N> void	AddStringArray(const char(&Name)[N], const TArray<FString> Values);
	FPayload				Done();

private:
	TInlineBuffer<Size>		Buffer;
	FCborWriter				CborWriter = { Buffer };
};

////////////////////////////////////////////////////////////////////////////////
template <int Size>
inline TPayloadBuilder<Size>::TPayloadBuilder(EStatusCode StatusCode)
{
	CborWriter.OpenMap();
	AddInteger("$status", int32(StatusCode));
}

////////////////////////////////////////////////////////////////////////////////
template <int Size>
template <int N>
inline TPayloadBuilder<Size>::TPayloadBuilder(const char (&Path)[N])
{
	CborWriter.OpenMap();
	AddString("$request", "GET", 3);
	AddString("$path", Path, N - 1);
}

////////////////////////////////////////////////////////////////////////////////
template <int Size>
template <int N>
inline void TPayloadBuilder<Size>::AddInteger(const char (&Name)[N], int64 Value)
{
	CborWriter.WriteString(Name, N - 1);
	CborWriter.WriteInteger(Value);
}

////////////////////////////////////////////////////////////////////////////////
template<int Size>
template<int N>
inline void TPayloadBuilder<Size>::AddStringArray(const char(&Name)[N], const TArray<FString> Values)
{
	CborWriter.WriteString(Name, N - 1);
	CborWriter.OpenArray();
	for (const auto& String : Values)
	{
		CborWriter.WriteString(*String);
	}
	CborWriter.Close();
}

////////////////////////////////////////////////////////////////////////////////
template <int Size>
template <int N>
inline void TPayloadBuilder<Size>::AddString(
	const char (&Name)[N],
	const char* Value,
	int Length)
{
	CborWriter.WriteString(Name, N - 1);
	CborWriter.WriteString(Value, Length);
}

////////////////////////////////////////////////////////////////////////////////
template <int Size>
inline FPayload TPayloadBuilder<Size>::Done()
{
	CborWriter.Close();
	return { Buffer->GetData(), uint32(Buffer->GetSize()) };
}



////////////////////////////////////////////////////////////////////////////////
class FResponse
{
public:
	EStatusCode		GetStatusCode() const;
	int64			GetInteger(const char* Key, int64 Default) const;
	template <int N>
	FStringView		GetString(const char* Key, const char (&Default)[N]) const;
	const uint8*	GetData() const;
	uint32			GetSize() const;
	uint8*			Reserve(uint32 Size);

private:
	template <typename Type, typename LambdaType>
	Type			GetValue(const char* Key, Type Default, LambdaType&& Lambda) const;
	TArray<uint8>	Buffer;
};

////////////////////////////////////////////////////////////////////////////////
inline EStatusCode FResponse::GetStatusCode() const
{
	int32 Code = int32(GetInteger("$status", 0));
	return Code ? EStatusCode(Code) : EStatusCode::Unknown;
}

////////////////////////////////////////////////////////////////////////////////
inline const uint8* FResponse::GetData() const
{
	return Buffer.GetData();
}

////////////////////////////////////////////////////////////////////////////////
inline uint32 FResponse::GetSize() const
{
	return Buffer.Num();
}

////////////////////////////////////////////////////////////////////////////////
inline uint8* FResponse::Reserve(uint32 Size)
{
	Buffer.SetNumUninitialized(Size);
	return Buffer.GetData();
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type, typename LambdaType>
inline Type FResponse::GetValue(const char* Key, Type Default, LambdaType&& Lambda) const
{
	FCborReader CborReader(Buffer.GetData(), Buffer.Num());
	FCborContext Context;

	if (!CborReader.ReadNext(Context) || Context.GetType() != ECborType::Map)
	{
		return Default;
	}

	while (true)
	{
		// Read key
		if (!CborReader.ReadNext(Context) || Context.GetType() != ECborType::String)
		{
			return Default;
		}

		// Check the key
		FStringView String = Context.AsString();
		bool bIsTarget = (String.Compare(Key) == 0);

		// Read value
		if (!CborReader.ReadNext(Context))
		{
			return Default;
		}

		if (bIsTarget)
		{
			return Lambda(Context);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
inline int64 FResponse::GetInteger(const char* Key, int64 Default) const
{
	return GetValue(
		Key,
		Default,
		[] (const FCborContext& Context)
		{
			return Context.AsInteger();
		}
	);
}

////////////////////////////////////////////////////////////////////////////////
template <int N>
inline FStringView FResponse::GetString(const char* Key, const char (&Default)[N]) const
{
	FStringView DefaultView(Default, N - 1);
	return GetValue(
		Key,
		DefaultView,
		[DefaultView] (const FCborContext& Context)
		{
			if (Context.GetType() == ECborType::String)
			{
				return Context.AsString();
			}

			return DefaultView;
		}
	);
}

/* vim: set noexpandtab : */
