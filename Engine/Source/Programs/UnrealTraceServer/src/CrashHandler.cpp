// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "CrashHandler.h"
#include "Foundation.h"
#include "Version.h"

#if TS_USING(TS_PLATFORM_WINDOWS)
#	include <dbghelp.h>
#elif TS_USING(TS_PLATFORM_LINUX) || TS_USING(TS_PLATFORM_MAC)
#	include <signal.h>
#	if !defined(TS_WITH_MUSL)
#		include <execinfo.h>
#		include <cxxabi.h>
#		define TS_HAVE_BACKTRACE 1
#	endif
#endif



// {{{1 shared state -----------------------------------------------------------

static char GCrashReportPath[4096];

#if TS_USING(TS_PLATFORM_WINDOWS)
static wchar_t GCrashReportPathW[4096];
static volatile LONG GInCrashHandler = 0;
#elif TS_USING(TS_PLATFORM_LINUX) || TS_USING(TS_PLATFORM_MAC)
static volatile sig_atomic_t GInCrashHandler = 0;
#define CRASH_ALT_STACK_SIZE (64 * 1024)
static uint8 GAltStack[CRASH_ALT_STACK_SIZE];
#endif



// {{{1 helpers ----------------------------------------------------------------

static const char KSeparator[] =
	"================================================================================\n";

////////////////////////////////////////////////////////////////////////////////
static void WriteLine(FILE* File, const char* Line)
{
	if (File != nullptr)
		fputs(Line, File);
	fputs(Line, stderr);
}

////////////////////////////////////////////////////////////////////////////////
static const char* GetVersionString()
{
	static char VersionBuf[32] = {};
	if (VersionBuf[0] == '\0')
	{
		snprintf(VersionBuf, sizeof(VersionBuf), "%d.%d%s",
			TS_VERSION_PROTOCOL,
			TS_VERSION_MINOR,
#if TS_USING(TS_BUILD_DEBUG)
			" [debug]"
#else
			""
#endif
		);
	}
	return VersionBuf;
}


// {{{1 windows ----------------------------------------------------------------

#if TS_USING(TS_PLATFORM_WINDOWS)

////////////////////////////////////////////////////////////////////////////////
static const char* ExceptionCodeToString(DWORD Code)
{
	switch (Code)
	{
		case EXCEPTION_ACCESS_VIOLATION:		return "Access Violation";
		case EXCEPTION_STACK_OVERFLOW:			return "Stack Overflow";
		case EXCEPTION_ILLEGAL_INSTRUCTION:		return "Illegal Instruction";
		case EXCEPTION_ARRAY_BOUNDS_EXCEEDED:	return "Array Bounds Exceeded";
		case EXCEPTION_INT_DIVIDE_BY_ZERO:		return "Integer Divide by Zero";
		case EXCEPTION_FLT_DIVIDE_BY_ZERO:		return "Float Divide by Zero";
		case EXCEPTION_DATATYPE_MISALIGNMENT:	return "Datatype Misalignment";
		case EXCEPTION_BREAKPOINT:				return "Breakpoint";
		default:								return "Unknown Exception";
	}
}

////////////////////////////////////////////////////////////////////////////////
static LONG WINAPI CrashFilter(EXCEPTION_POINTERS* Info)
{
	if (InterlockedExchange(&GInCrashHandler, 1))
		return EXCEPTION_EXECUTE_HANDLER;

	FILE* File = _wfopen(GCrashReportPathW, L"w");
	if (!File)
	{
		WriteLine(File, "Failed to open crash report file for writing:");
		WriteLine(File, GCrashReportPath);
	}

	// Header
	WriteLine(File, KSeparator);
	WriteLine(File, "UnrealTraceServer Crash Report\n");
	WriteLine(File, KSeparator);

	// Timestamp
	{
		SYSTEMTIME St = {};
		GetSystemTime(&St);
		char Buf[64];
		snprintf(Buf, sizeof(Buf), "Time:    %04d-%02d-%02d %02d:%02d:%02d UTC\n",
			(int)St.wYear, (int)St.wMonth, (int)St.wDay,
			(int)St.wHour, (int)St.wMinute, (int)St.wSecond);
		WriteLine(File, Buf);
	}

	// Version
	{
		char Buf[64];
		snprintf(Buf, sizeof(Buf), "Version: %s\n", GetVersionString());
		WriteLine(File, Buf);
	}

	// Error
	{
		DWORD Code = Info->ExceptionRecord->ExceptionCode;
		ULONG_PTR* Params = Info->ExceptionRecord->ExceptionInformation;
		char Buf[128];
		snprintf(Buf, sizeof(Buf), "Error:   %s (code 0x%08lX)\n",
			ExceptionCodeToString(Code), (unsigned long)Code);
		WriteLine(File, Buf);

		if (Code == EXCEPTION_ACCESS_VIOLATION || Code == EXCEPTION_IN_PAGE_ERROR)
		{
			const char* AccessType = (Params[0] == 0) ? "read"
			                       : (Params[0] == 1) ? "write"
			                       :                    "execute";
			snprintf(Buf, sizeof(Buf), "Detail:  %s at 0x%016llx\n",
				AccessType, (unsigned long long)Params[1]);
			WriteLine(File, Buf);
		}
	}

	// Callstack
	WriteLine(File, "\nCallstack:\n");
	{
		void* Frames[62];
		USHORT Count = CaptureStackBackTrace(0, 62, Frames, nullptr);

		HANDLE Process = GetCurrentProcess();
		SymInitialize(Process, nullptr, TRUE);

		for (USHORT i = 0; i < Count; ++i)
		{
			char SymBuf[sizeof(SYMBOL_INFO) + MAX_SYM_NAME] = {};
			SYMBOL_INFO* Sym = (SYMBOL_INFO*)SymBuf;
			Sym->SizeOfStruct = sizeof(SYMBOL_INFO);
			Sym->MaxNameLen = MAX_SYM_NAME;

			DWORD64 Addr = (DWORD64)Frames[i];
			DWORD64 Displacement = 0;
			char LineBuf[512];

			if (SymFromAddr(Process, Addr, &Displacement, Sym))
			{
				IMAGEHLP_LINE64 Line = {};
				Line.SizeOfStruct = sizeof(IMAGEHLP_LINE64);
				DWORD LineDisp = 0;
				if (SymGetLineFromAddr64(Process, Addr, &LineDisp, &Line))
				{
					snprintf(LineBuf, sizeof(LineBuf),
						"  [%2d] %s + 0x%llx  [%s:%lu]\n",
						(int)i, Sym->Name, (unsigned long long)Displacement,
						Line.FileName, (unsigned long)Line.LineNumber);
				}
				else
				{
					snprintf(LineBuf, sizeof(LineBuf),
						"  [%2d] %s + 0x%llx\n",
						(int)i, Sym->Name, (unsigned long long)Displacement);
				}
			}
			else
			{
				snprintf(LineBuf, sizeof(LineBuf),
					"  [%2d] 0x%016llx\n",
					(int)i, (unsigned long long)Addr);
			}
			WriteLine(File, LineBuf);
		}

		SymCleanup(Process);
	}

	WriteLine(File, KSeparator);
	if (File != nullptr)
		fclose(File);
	return EXCEPTION_EXECUTE_HANDLER;
}

#endif // TS_PLATFORM_WINDOWS



// {{{1 posix ------------------------------------------------------------------

#if TS_USING(TS_PLATFORM_LINUX) || TS_USING(TS_PLATFORM_MAC)

////////////////////////////////////////////////////////////////////////////////
static const char* SignalToString(int Sig)
{
	switch (Sig)
	{
		case SIGSEGV: return "Segmentation Fault";
		case SIGABRT: return "Abort";
		case SIGFPE:  return "Floating Point Exception";
		case SIGILL:  return "Illegal Instruction";
		case SIGBUS:  return "Bus Error";
		default:      return "Unknown Signal";
	}
}

////////////////////////////////////////////////////////////////////////////////
#if defined(TS_HAVE_BACKTRACE)
static void WriteFrame(FILE* File, int Index, void* Addr, const char* RawSymbol)
{
	char Line[512];
	bool Written = false;

	if (RawSymbol != nullptr)
	{
		// Attempt to extract and demangle the symbol name from the
		// platform-specific backtrace_symbols format.
		const char* MangleStart = nullptr;
		const char* MangleEnd   = nullptr;

#if TS_USING(TS_PLATFORM_LINUX)
		// Linux: "./binary(mangled+0xoffset) [0xaddr]"
		MangleStart = strchr(RawSymbol, '(');
		if (MangleStart != nullptr)
		{
			++MangleStart;
			MangleEnd = strchr(MangleStart, '+');
			if (MangleEnd == nullptr)
				MangleEnd = strchr(MangleStart, ')');
		}
#elif TS_USING(TS_PLATFORM_MAC)
		// macOS: "N   binary  0xaddr  symbol + offset"
		// Skip 3 whitespace-delimited tokens to reach the symbol name
		const char* P = RawSymbol;
		for (int Token = 0; Token < 3 && *P != '\0'; ++Token)
		{
			while (*P == ' ') ++P;
			while (*P != '\0' && *P != ' ') ++P;
		}
		while (*P == ' ') ++P;
		MangleStart = P;
		const char* PlusSign = strrchr(MangleStart, '+');
		if (PlusSign != nullptr && PlusSign > MangleStart)
		{
			MangleEnd = PlusSign - 1;
			while (MangleEnd > MangleStart && *(MangleEnd - 1) == ' ')
				--MangleEnd;
		}
		else
		{
			MangleEnd = MangleStart + strlen(MangleStart);
		}
#endif

		if (MangleStart != nullptr && MangleEnd > MangleStart)
		{
			char MangleBuf[256] = {};
			size_t MangleLen = (size_t)(MangleEnd - MangleStart);
			if (MangleLen >= sizeof(MangleBuf))
				MangleLen = sizeof(MangleBuf) - 1;
			memcpy(MangleBuf, MangleStart, MangleLen);

			int Status = -1;
			char* Demangled = abi::__cxa_demangle(MangleBuf, nullptr, nullptr, &Status);
			if (Status == 0 && Demangled != nullptr)
			{
				snprintf(Line, sizeof(Line), "  [%2d] %s\n", Index, Demangled);
				free(Demangled);
				Written = true;
			}
		}

		if (!Written)
		{
			snprintf(Line, sizeof(Line), "  [%2d] %s\n", Index, RawSymbol);
			Written = true;
		}
	}

	if (!Written)
		snprintf(Line, sizeof(Line), "  [%2d] %p\n", Index, Addr);

	WriteLine(File, Line);
}
#endif // TS_HAVE_BACKTRACE

////////////////////////////////////////////////////////////////////////////////
static void CrashSignalHandler(int Sig, siginfo_t* /*Info*/, void* /*Context*/)
{
	if (GInCrashHandler)
		_exit(2);
	GInCrashHandler = 1;

	FILE* File = fopen(GCrashReportPath, "w");
	if (!File)
	{
		WriteLine(File, "Failed to open crash report file for writing:");
		WriteLine(File, GCrashReportPath);
	}

	// Header
	WriteLine(File, KSeparator);
	WriteLine(File, "UnrealTraceServer Crash Report\n");
	WriteLine(File, KSeparator);

	// Timestamp
	{
		struct timespec Ts = {};
		clock_gettime(CLOCK_REALTIME, &Ts);
		struct tm Tm = {};
		gmtime_r(&Ts.tv_sec, &Tm);
		char Buf[64];
		snprintf(Buf, sizeof(Buf), "Time:    %04d-%02d-%02d %02d:%02d:%02d UTC\n",
			Tm.tm_year + 1900, Tm.tm_mon + 1, Tm.tm_mday,
			Tm.tm_hour, Tm.tm_min, Tm.tm_sec);
		WriteLine(File, Buf);
	}

	// Version
	{
		char Buf[64];
		snprintf(Buf, sizeof(Buf), "Version: %s\n", GetVersionString());
		WriteLine(File, Buf);
	}

	// Error
	{
		char Buf[128];
		snprintf(Buf, sizeof(Buf), "Error:   %s (signal %d)\n",
			SignalToString(Sig), Sig);
		WriteLine(File, Buf);
	}

	// Callstack
	WriteLine(File, "\nCallstack:\n");
#if defined(TS_HAVE_BACKTRACE)
	{
		void* Frames[32];
		int Count = backtrace(Frames, 32);
		char** Symbols = backtrace_symbols(Frames, Count);
		for (int i = 0; i < Count; ++i)
			WriteFrame(File, i, Frames[i], Symbols != nullptr ? Symbols[i] : nullptr);
		free(Symbols);
	}
#else
	WriteLine(File, "  (callstack unavailable on musl builds)\n");
#endif

	WriteLine(File, KSeparator);
	if (File != nullptr)
		fclose(File);
	_exit(1);
}

#endif // TS_PLATFORM_LINUX || TS_PLATFORM_MAC



// {{{1 install ----------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
void InstallCrashHandler(const FPath& LogDir)
{
	// Pre-compute the crash report path into static buffers once at startup,
	// so it is safe to use during crash handling without any allocation.
	FString Dir = fs::ToFString(LogDir);
	snprintf(GCrashReportPath, sizeof(GCrashReportPath),
		"%s/CrashReport.txt", Dir.c_str());

#if TS_USING(TS_PLATFORM_WINDOWS)
	std::wstring WDir = LogDir.wstring();
	swprintf(GCrashReportPathW, 4096, L"%ls/CrashReport.txt", WDir.c_str());
	SetUnhandledExceptionFilter(CrashFilter);

#elif TS_USING(TS_PLATFORM_LINUX) || TS_USING(TS_PLATFORM_MAC)
	stack_t AltStack = {};
	AltStack.ss_sp   = GAltStack;
	AltStack.ss_size = CRASH_ALT_STACK_SIZE;
	sigaltstack(&AltStack, nullptr);

	struct sigaction Action = {};
	Action.sa_sigaction = CrashSignalHandler;
	Action.sa_flags     = SA_SIGINFO | SA_ONSTACK;
	sigemptyset(&Action.sa_mask);

	sigaction(SIGSEGV, &Action, nullptr);
	sigaction(SIGABRT, &Action, nullptr);
	sigaction(SIGFPE,  &Action, nullptr);
	sigaction(SIGILL,  &Action, nullptr);
	sigaction(SIGBUS,  &Action, nullptr);
#endif
}

/* vim: set noexpandtab foldlevel=1 : */
