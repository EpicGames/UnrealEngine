// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Foundation.h"

// Call once at startup after the log directory is known.
// Installs platform crash handlers and pre-computes the crash report path.
void InstallCrashHandler(const FPath& LogDir);

/* vim: set noexpandtab : */
