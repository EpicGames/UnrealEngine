// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

////////////////////////////////////////////////////////////////////////////////
#define MoveTemp(x) std::move(x)
#define check(x)

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
struct TArray
	: protected std::vector<Type>
{
	using Super = std::vector<Type>;
	using Super::Super;
	using Super::operator [];
	using Super::begin;
	using Super::end;

	void		Add(const Type& Value)				{ return Super::push_back(Value); }
	void		SetNum(int32 Num)					{ return Super::resize(Num); }
	int32		Num() const							{ return int32(Super::size()); }
	Type*		GetData()							{ return &(this->operator [] (0)); }
	const Type* GetData() const						{ return &(this->operator [] (0)); }
	void		Empty()								{ return Super::clear(); }
	void		SetNumUninitialized(int Num)		{ return Super::resize(Num); }
	void		Append(const void* Data, int Num)	{ Super::insert(Super::end(), (const Type*)Data, ((const Type*)Data) + Num); }
	Type&		Last()								{ return Super::back(); }
	template<typename Lambda>
	uint32		RemoveIf(Lambda&& Pred);
	template<typename Lambda> 
	bool		FindOrAdd(const Type& Item, Lambda&& Pred);
};

template<typename Type>
template<typename Lambda>
inline uint32 TArray<Type>::RemoveIf(Lambda&& Pred) 
{
	uint32 Removed = 0;
	for (auto it = begin(); it != end();) 
	{
		if (Pred(*it))
		{
			Super::erase(it); 
			++Removed;
		}
		else
		{
			++it;
		}
	}
	return Removed;
}

template<typename Type>
template<typename Lambda>
inline bool TArray<Type>::FindOrAdd(const Type& Item, Lambda&& Compare)
{
	for (auto It = begin(); It != end(); ++It)
	{
		if (Compare(*It, Item) == 0)
		{
			return false;
		}
	}
	Super::push_back(Item);
	return true;
}

////////////////////////////////////////////////////////////////////////////////
struct FStringView
	: protected std::basic_string_view<char>
{
	typedef std::basic_string_view<char> Super;
	using Super::Super;
	using Super::operator [];

	const char* GetData() const			{ return Super::data(); }
	uint32 Len() const					{ return uint32(Super::size()); }
	
	int Compare(const char* Rhs) const	{ return Super::compare(Rhs); }
	void RemovePrefix(uint32 n)			{ Super::remove_prefix(n); }
};

////////////////////////////////////////////////////////////////////////////////
struct FString
	: public std::string
{
	typedef std::string Super;
	using Super::Super;
	using Super::operator +=;

					FString(const std::string& Rhs) : Super(Rhs)	{}
					FString(std::string&& Rhs) : Super(Rhs)			{}
					FString(FStringView Rhs) : Super(Rhs.GetData(), Rhs.Len())			{}
	const char*		operator * () const								{ return Super::c_str(); }
	void			operator += (const FStringView& Rhs)			{ Super::operator += ((FStringView::Super&)(Rhs)); }
};

////////////////////////////////////////////////////////////////////////////////
typedef std::filesystem::path FPath;


////////////////////////////////////////////////////////////////////////////////
namespace fs
{
	using namespace std::filesystem;

	inline FString ToFString(const path& Path)
	{
#if TS_USING(TS_PLATFORM_WINDOWS)
		std::wstring String = Path.wstring();
		size_t OutSize = WideCharToMultiByte(
			CP_UTF8, 0,
			String.c_str(), -1,
			nullptr, 0,
			nullptr, nullptr);

		std::string Ret;
		Ret.resize(OutSize - 1);
		WideCharToMultiByte(
			CP_UTF8, 0,
			String.c_str(), -1,
			Ret.data(), int(Ret.size()),
			nullptr, nullptr);

		return FString(Ret);
#else
		return Path.string();
#endif
	}
}



////////////////////////////////////////////////////////////////////////////////
#if TS_USING(TS_PLATFORM_WINDOWS)
struct FWinApiStr
{
	FWinApiStr(const char* Utf8)
	{
		int32 BufferSize = MultiByteToWideChar(CP_UTF8, 0, Utf8, -1, nullptr, 0);
		Buffer = new wchar_t[BufferSize];
		MultiByteToWideChar(CP_UTF8, 0, Utf8, -1, Buffer, BufferSize);
	}

	~FWinApiStr()
	{
		delete[] Buffer;
	}

	operator LPCWSTR () const
	{
		return Buffer;
	}

private:
	wchar_t* Buffer = nullptr;
};
#endif // TS_PLATFORM_WINDOWS

////////////////////////////////////////////////////////////////////////////////

struct FGuid
{
	uint32 Bits[4] = { 0, 0, 0, 0 };

	static bool ParseGuid(FStringView Source, FGuid& OutGuid)
	{
		std::basic_string_view<char> SourceView(Source.GetData(), Source.Len());
		std::basic_string_view<char> A(SourceView.substr(0, 8)),
			B(SourceView.substr(8, 8)),
			C(SourceView.substr(16, 8)),
			D(SourceView.substr(24, 8));

		if (auto result = std::from_chars(A.data(), A.data() + A.length(), OutGuid.Bits[0], 16); result.ptr != (A.data() + A.length()))
		{
			return false;
		}
		if (auto result = std::from_chars(B.data(), B.data() + B.length(), OutGuid.Bits[1], 16); result.ptr != (B.data() + B.length()))
		{
			return false;
		}
		if (auto result = std::from_chars(C.data(), C.data() + C.length(), OutGuid.Bits[2], 16); result.ptr != (C.data() + C.length()))
		{
			return false;
		}
		if (auto result = std::from_chars(D.data(), D.data() + D.length(), OutGuid.Bits[3], 16); result.ptr != (D.data() + D.length()))
		{
			return false;
		}
		
		return true;
	}

	static bool Equal(const FGuid& Rhs, const FGuid Lhs)
	{
		return Rhs.Bits[0] == Lhs.Bits[0] && Rhs.Bits[1] == Lhs.Bits[1] 
			&& Rhs.Bits[2] == Lhs.Bits[2] && Rhs.Bits[3] == Lhs.Bits[3];
	}
};

/* vim: set noexpandtab : */
