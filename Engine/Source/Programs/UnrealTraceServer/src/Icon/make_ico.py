# Copyright Epic Games, Inc. All Rights Reserved.

from PIL import Image

def MakeIcon(prefix, output):
    sizes = [256, 64, 48, 32, 24, 16]
    images = []
    for size in sizes:
        print(f"Loading {prefix}_{size}.png")
        image = Image.open(f"{prefix}_{size}.png")
        images.append(image)
    print(f"Creating {output}")
    images[0].save(f"{output}", append_images=images)

MakeIcon("icon", "icon.ico")
