// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "InstanceInfo.h"

////////////////////////////////////////////////////////////////////////////////
void FInstanceInfo::Set()
{
	Version = CurrentVersion;
#if TS_USING(TS_PLATFORM_WINDOWS)
	Pid = GetCurrentProcessId();
#elif TS_USING(TS_PLATFORM_LINUX) || TS_USING(TS_PLATFORM_MAC)
	Pid = getpid();
#endif
	Published.fetch_add(1, std::memory_order_release);
}

////////////////////////////////////////////////////////////////////////////////
void FInstanceInfo::WaitForReady() const
{
	// Spin until this instance info is published (by another process)
#if TS_USING(TS_PLATFORM_WINDOWS)
	for (;; Sleep(0))
#else
	for (;; sched_yield())
#endif
	{
		if (Published.load(std::memory_order_acquire))
		{
			break;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
bool FInstanceInfo::IsOlder() const
{
	// Decide which is older; this compiled code or the instance we have a
	// pointer to.
	bool bIsOlder = false;
	bIsOlder |= (Version < FInstanceInfo::CurrentVersion);
	return bIsOlder;
}

////////////////////////////////////////////////////////////////////////////////
bool FInstanceInfo::AddSponsor(uint32 Pid)
{
	if (!Pid)
	{
		return true;
	}

	for (auto& PidEntry : SponsorPids)
	{
		if (uint32_t ThisPid = PidEntry.load(std::memory_order_relaxed); !ThisPid)
		{
			if (PidEntry.compare_exchange_strong(ThisPid, Pid, std::memory_order_relaxed))
			{
				return true;
			}
		}
	}
	return false;
}

