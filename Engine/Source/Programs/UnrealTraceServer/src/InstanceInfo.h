// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Version.h"

////////////////////////////////////////////////////////////////////////////////
struct FInstanceInfo
{
public:
	static constexpr uint32	NumSponsorQueueSlots = 128;
	static const uint32	CurrentVersion =
#if TS_USING(TS_BUILD_DEBUG)
		0x8000'0000 |
#endif
		((TS_VERSION_PROTOCOL & 0xffff) << 16) | (TS_VERSION_MINOR & 0xffff);

	void				Set();
	void				WaitForReady() const;
	bool				IsOlder() const;
	bool				AddSponsor(uint32 Pid);

	std::atomic<uint32> Published;
	uint32				Version;
	uint32				Pid;
	// Queue slots used to add sponsor processes. The running instance regularly 
	// polls these slots and moves the pid to an internal representation.
	std::atomic<uint32>	SponsorPids[NumSponsorQueueSlots];
};


