// Copyright Epic Games, Inc. All Rights Reserved.
#include "Pch.h"
#include "Lifetime.h"
#include "InstanceInfo.h"
#include "Logging.h"
#include "StoreService.h"
#include "StoreSettings.h"

#if TS_USING(TS_PLATFORM_LINUX) || TS_USING(TS_PLATFORM_MAC)
#include <signal.h>
#endif

#if TS_USING(TS_PLATFORM_WINDOWS)
static HANDLE GQuitEvent = NULL;
static const wchar_t* GQuitEventName = L"Local\\UnrealTraceEvent";
#endif

constexpr uint32 GSponsorCheckFreqSecs = 5;

////////////////////////////////////////////////////////////////////////////////
FLifetime::FLifetime(asio::io_context& IoContext, 
					 FStoreService* InStoreService, 
					 FStoreSettings* InSettings, 
					 FInstanceInfo* InInstanceInfo)
	: FAsioTickable(IoContext)
	, StoreService(InStoreService)
	, Settings(InSettings)
	, InstanceInfo(InInstanceInfo)
{
	check(StoreService);
	check(InstanceInfo);
	check(Settings);

#if TS_USING(TS_PLATFORM_WINDOWS)
	GQuitEvent = OpenEventW(EVENT_MODIFY_STATE, FALSE, GQuitEventName);
	check(GQuitEvent != NULL);
#endif

	StartTick(GSponsorCheckFreqSecs * 1000);
}

////////////////////////////////////////////////////////////////////////////////
void FLifetime::OnTick()
{
	CheckNewSponsors(InstanceInfo);
	// We need to call IsAnySponsorsActive regularly even if sponsored mode is not 
	// activated. This is because otherwise the SponsorHandles array would build 
	// up indefinetly and the risk of reuse of pid increase.
	if (!IsAnySponsorActive() && Settings->Sponsored)
	{
		if (ShutdownStoreIfNoConnections())
		{
			TS_LOG("Terminating server, no sponsors or connections active.");
#if TS_USING(TS_PLATFORM_WINDOWS)
			SetEvent(GQuitEvent);
#else
			kill(getpid(), SIGTERM);
#endif
			StopTick();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
void FLifetime::CheckNewSponsors(FInstanceInfo* InstanceInfo)
{
	check(InstanceInfo);
	for (auto& PidEntry : InstanceInfo->SponsorPids)
	{
		if (uint32_t ThisPid = PidEntry.load(std::memory_order_relaxed))
		{
			if (PidEntry.compare_exchange_strong(ThisPid, 0))
			{
				AddPid(ThisPid);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
void FLifetime::AddPid(uint32 Pid)
{
	if (Pid == 0)
	{
		return;
	}
#if TS_USING(TS_PLATFORM_WINDOWS)
	FProcHandle ProcessHandle = OpenProcess(PROCESS_QUERY_LIMITED_INFORMATION | SYNCHRONIZE, FALSE, Pid);
	if (!ProcessHandle)
	{
		TS_LOG("Failed to open sponsor process (error code: %d), will not be able to use as sponsor.", GetLastError());
		return;
	}
#else
	FProcHandle ProcessHandle = FProcHandle(intptr_t(Pid));
#endif
	SponsorHandles.FindOrAdd(ProcessHandle, [](FProcHandle& Handle, const FProcHandle& New) {return Handle == New ? 0 : 1; });
}

////////////////////////////////////////////////////////////////////////////////
bool FLifetime::ShutdownStoreIfNoConnections()
{
	return StoreService->ShutdownIfNoConnections();
}

////////////////////////////////////////////////////////////////////////////////
bool FLifetime::IsAnySponsorActive()
{
	bool bKeepAlive = false;
	for (auto& Handle : SponsorHandles)
	{
#if TS_USING(TS_PLATFORM_WINDOWS)
		DWORD ExitCode;
		const bool Result = GetExitCodeProcess(Handle, &ExitCode);
		if (Result && ExitCode == STILL_ACTIVE)
		{
			bKeepAlive = true;
		}
		else
		{
			Handle = 0;
		}
#else
		int Pid = int(intptr_t(Handle));
		if (kill(pid_t(Pid), 0) == 0)
		{
			bKeepAlive = true;
		}
		else
		{
			Handle = 0;
		}
#endif
	}

	// Remove inactive processes
	SponsorHandles.RemoveIf([](const FProcHandle& Handle) { return Handle == 0; });
	return bKeepAlive;
}

