// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Foundation.h"
#include "AsioTickable.h"

////////////////////////////////////////////////////////////////////////////////

class FLifetime : public FAsioTickable
{
public:
						FLifetime(
							asio::io_context& IoContext, 
							class FStoreService* Service, 
							class FStoreSettings* Settings, 
							struct FInstanceInfo* InstanceInfo
						);

private:
	void				CheckNewSponsors(struct FInstanceInfo* InstanceInfo);
	void				AddPid(uint32);
	bool				ShutdownStoreIfNoConnections();
	bool				IsAnySponsorActive();
	virtual void		OnTick() override;

	using FProcHandle = void*;

	TArray<FProcHandle> SponsorHandles;
	class FStoreService* StoreService;
	class FStoreSettings* Settings;
	struct FInstanceInfo* InstanceInfo;
};

