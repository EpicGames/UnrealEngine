// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "Logging.h"
#include "Foundation.h"

#if TS_USING(TS_PLATFORM_LINUX) || TS_USING(TS_PLATFORM_MAC)
#include <cstdarg>
#endif

////////////////////////////////////////////////////////////////////////////////
FLogging* FLogging::Instance = nullptr;

////////////////////////////////////////////////////////////////////////////////
FLogging::FLogging(FPath& InPath)
	: Path(InPath)
{
	// Find where the logs should be written to. Make sure it exists.
	FPath LogDir = Path.parent_path();
	FPath Stem = Path.stem();

	// Expected format for log files
	const FString FilenameFormat = Stem.string() + "_%d.log";

	// Fetch all existing logs.
	struct FExistingLog
	{
		FPath	Path;
		uint32	Index;

		int32 operator < (const FExistingLog& Rhs) const
		{
			return Index < Rhs.Index;
		}
	};
	std::vector<FExistingLog> ExistingLogs;
	if (std::filesystem::is_directory(LogDir))
	{
		
		for (const auto& DirItem : std::filesystem::directory_iterator(LogDir))
		{
			int32 Index = -1;
			std::string StemUtf8 = DirItem.path().filename().string();
			sscanf(StemUtf8.c_str(), *FilenameFormat, &Index);
			if (Index >= 0)
			{
				ExistingLogs.push_back({ DirItem.path(), uint32(Index) });
			}
		}
	}

	// Sort and try and tidy up old logs.
	static int32 MaxLogs = 12; // plus one new one
	std::sort(ExistingLogs.begin(), ExistingLogs.end());
	for (int32 i = 0, n = int32(ExistingLogs.size() - MaxLogs); i < n; ++i)
	{
		std::error_code ErrorCode;
		std::filesystem::remove(ExistingLogs[i].Path, ErrorCode);
	}


	// Open the log file (note; can race other instances)
	uint32 LogIndex = ExistingLogs.empty() ? 0 : ExistingLogs.back().Index;
	for (uint32 n = LogIndex + 10; File == nullptr && LogIndex < n;)
	{
		++LogIndex;
		char LogName[128];
		snprintf(LogName, TS_ARRAY_COUNT(LogName), *FilenameFormat, LogIndex);
		FPath LogPath = LogDir / LogName;

#if TS_USING(TS_PLATFORM_WINDOWS)
		File = _wfopen(LogPath.c_str(), L"wbxN");
#else
		File = fopen(LogPath.c_str(), "wbx");
#endif
	}
}

////////////////////////////////////////////////////////////////////////////////
FLogging::~FLogging()
{
	if (File != nullptr)
	{
		fclose(File);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FLogging::LogImpl(const char* String) const
{
	if (File != nullptr)
	{
		fputs(String, File);
		fflush(File);
	}

	fputs(String, stdout);
#if TS_USING(TS_PLATFORM_WINDOWS) && TS_USING(TS_BUILD_DEBUG)
	OutputDebugString(String);
#endif
}

////////////////////////////////////////////////////////////////////////////////
void FLogging::Log(const char* Format, ...)
{
	va_list VaList;
	va_start(VaList, Format);

	char Buffer[320];
	vsnprintf(Buffer, TS_ARRAY_COUNT(Buffer), Format, VaList);
	Buffer[TS_ARRAY_COUNT(Buffer) - 1] = '\0';

	Instance->LogImpl(Buffer);

	va_end(VaList);
}

////////////////////////////////////////////////////////////////////////////////
FLoggingScope::FLoggingScope(FPath& InPath, const char* InFilename /* = nullptr */)
{
	FPath Path = InPath / (InFilename ? InFilename : "Server");
	if (FLogging::Instance != nullptr && FLogging::Instance->Path == Path)
	{
		return;
	}

	PreviousScope = FLogging::Instance;
	FLogging::Instance = new FLogging(Path);
}

////////////////////////////////////////////////////////////////////////////////
FLoggingScope::~FLoggingScope() 
{
	delete FLogging::Instance;
	FLogging::Instance = PreviousScope;
}
