// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Foundation.h"

////////////////////////////////////////////////////////////////////////////////
#define TS_LOG(Format, ...) \
	do { FLogging::Log(Format "\n", ##__VA_ARGS__); } while (false)

////////////////////////////////////////////////////////////////////////////////
class FLogging
{
public:
	static void			Log(const char* Format, ...);

private:
	friend struct FLoggingScope;
	FLogging(FPath& Path);
	~FLogging();
	FLogging(const FLogging&) = delete;
	FLogging(FLogging&&) = default;
	void				LogImpl(const char* String) const;
	static FLogging* Instance;
	FPath Path;
	FILE* File = nullptr;
};

////////////////////////////////////////////////////////////////////////////////
struct FLoggingScope
{
	FLoggingScope(FPath& Path, const char* LogFileName = nullptr);
	~FLoggingScope();

private: 
	FLogging* PreviousScope = nullptr;
};