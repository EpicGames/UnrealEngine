// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "InstanceInfo.h"
#include "Lifetime.h"
#include "Logging.h"
#include "StoreService.h"
#include "StoreSettings.h"
#include "Utils.h"
#include "Version.h"
#include "CrashHandler.h"

#include <cxxopts.hpp>

#if TS_USING(TS_BUILD_DEBUG)
#	include <thread>
#endif

#if TS_USING(TS_PLATFORM_LINUX) || TS_USING(TS_PLATFORM_MAC)
#	include <cstdarg>
#	include <ctime>
#	include <pthread.h>
#	include <pwd.h>
#	include <sched.h>
#	include <semaphore.h>
#	include <signal.h>
#	include <sys/file.h>
#	include <sys/mman.h>
#	include <sys/stat.h>
#	include <sys/wait.h>
#	include <unistd.h>
#endif

// Debug builds act as both the forker and the daemon. Set to TS_OFF to disable
// this behaviour.
#if TS_USING(TS_BUILD_DEBUG) //&&0
#	define TS_DAEMON_THREAD TS_ON
#else
#	define TS_DAEMON_THREAD TS_OFF
#endif

// Check for legacy lock files on Linux and Mac 
#define TS_LEGACY_LOCK_FILE TS_ON

// {{{1 misc -------------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FMmapScope
{
public:
								FMmapScope(void* InPtr, uint32 InLength=0);
								~FMmapScope();
	template <typename T> T*	As() const;

private:
	void*						Ptr;
	int32						Length;

private:
								FMmapScope() = delete;
								FMmapScope(const FMmapScope&) = delete;
								FMmapScope(const FMmapScope&&) = delete;
	FMmapScope&					operator = (const FMmapScope&) = delete;
	FMmapScope&					operator = (const FMmapScope&&) = delete;
};

////////////////////////////////////////////////////////////////////////////////
FMmapScope::FMmapScope(void* InPtr, uint32 InLength)
: Ptr(InPtr)
, Length(InLength)
{
}

////////////////////////////////////////////////////////////////////////////////
FMmapScope::~FMmapScope()
{
#if TS_USING(TS_PLATFORM_WINDOWS)
	UnmapViewOfFile(Ptr);
#elif TS_USING(TS_PLATFORM_LINUX) || TS_USING(TS_PLATFORM_MAC)
	munmap(Ptr, Length);
#endif
}

////////////////////////////////////////////////////////////////////////////////
template <typename T>
T* FMmapScope::As() const
{
	return (T*)Ptr;
}



////////////////////////////////////////////////////////////////////////////////
template <typename T>
struct TOnScopeExit
{
		TOnScopeExit()	= default;
		~TOnScopeExit()	{ (*Lambda)(); }
	T*	Lambda;
};

#define OnScopeExit(x) \
	auto TS_CONCAT(OnScopeExitFunc, __LINE__) = x; \
	TOnScopeExit<decltype(TS_CONCAT(OnScopeExitFunc, __LINE__))> TS_CONCAT(OnScopeExitInstance, __LINE__); \
	do { TS_CONCAT(OnScopeExitInstance, __LINE__).Lambda = &TS_CONCAT(OnScopeExitFunc, __LINE__); } while (0)



////////////////////////////////////////////////////////////////////////////////
static void GetUnrealTraceHome(FPath& Out, bool Make=false)
{
#if TS_USING(TS_PLATFORM_WINDOWS)
	wchar_t Buffer[MAX_PATH];
	auto Ok = SHGetFolderPathW(nullptr, CSIDL_LOCAL_APPDATA, nullptr, SHGFP_TYPE_CURRENT, Buffer);
	if (Ok != S_OK)
	{
		uint32 Ret = GetEnvironmentVariableW(L"USERPROFILE", Buffer, TS_ARRAY_COUNT(Buffer));
		if (Ret == 0 || Ret >= TS_ARRAY_COUNT(Buffer))
		{
			return;
		}
	}
	Out = Buffer;
	Out /= "UnrealEngine/Common/UnrealTrace";
#else
	if (const char* Home = getenv("HOME"))
	{
		Out = Home;
	}
	else
	{
		int UserId = getuid();
		const passwd* Passwd = getpwuid(UserId);
		if (Passwd)
		{
			Out = Passwd->pw_dir;
		}
		else 
		{
			Out = "/";
		}
	}
	Out /= "UnrealEngine/UnrealTrace";
#endif

	if (Make)
	{
		std::error_code ErrorCode;
		std::filesystem::create_directories(Out, ErrorCode);
	}
}

// {{{1 store ------------------------------------------------------------------

struct FOptions 
{
public:

	enum ParseResults : int {
		Result_Continue = 0,
		Result_MissingCommandError,
		Result_HelpRequested,
	};

	FOptions()
		: Options("UnrealTraceServer", FOptions::GetMainHelpString())
		, CurrentCommand(nullptr)
	{
		// Positional arguments
		Options.add_options()
			("command", "Command to execute", cxxopts::value<std::string>()->default_value(""))
			;
		Options.parse_positional({ "command" });
		Options.positional_help("<cmd>");

		Options.add_options()
			("h,help", "Prints help message for each command")
			("d,detach", "Detach from console when started.");

		// Fork and daemon options
		Options.add_options("settings")
			("storedir", "Default store directory", cxxopts::value<std::string>()->default_value(""))
			("port", "TCP port to serve the store on", cxxopts::value<int>()->default_value("0"))
			("recport", "TCP port for the recorder to listen on", cxxopts::value<int>()->default_value("0"))
			;

		// AddProc options
		Options.add_options("sponsor")
			("sponsor", "Pid to add as sponsor. Required if running in sponsored mode.", cxxopts::value<uint32>()->default_value("0"))
			("sponsor-mode", "Set sponsor mode. Zero to disable, non-zero to enable.", cxxopts::value<uint32>());
	}

	ParseResults Parse(int ArgC, char** ArgV)
	{
		Parsed = Options.parse(ArgC, ArgV);
		const bool bCommandOk = Parsed["command"].count() == 1;
		const bool bHelp = Parsed["help"].count() > 0;

		// Check for valid command
		std::string ParsedCommand = Parsed["command"].as<std::string>();
		for (FCommandHelp* CommandIt = CommandHelp; CommandIt->Command != nullptr; ++CommandIt)
		{
			if (strcmp(ParsedCommand.c_str(), CommandIt->Command) == 0)
			{
				CurrentCommand = CommandIt;
				break;
			}
		}

		// Update current command help
		if (!CurrentCommand)
		{
			std::stringstream CurrentCommandHelp;
			CurrentCommandHelp << "<Command>\n\nCommands:\n";
			for (FCommandHelp* CommandIt = CommandHelp; CommandIt->Command != nullptr; ++CommandIt)
			{
				CurrentCommandHelp << "  " << CommandIt->Command << "\t" << CommandIt->ShortText << "\n";
			}
			Options.positional_help(CurrentCommandHelp.str());
		}
		else
		{
			std::stringstream CurrentCommandHelp;
			CurrentCommandHelp << CurrentCommand->Command << "\n\nCommand:\n  "
				<< CurrentCommand->ShortText << CurrentCommand->LongText << "\n";
			Options.positional_help(CurrentCommandHelp.str());
		}

		if (!bCommandOk)
		{
			fputs("Error, unknown command specified. Note that command matching is case sensitive.\n", stderr);
			PrintHelp();
			return Result_MissingCommandError;
		}
		else if (bHelp)
		{
			PrintHelp();
			return Result_HelpRequested;
		}
		
		return Result_Continue;
	}

	void ApplyToSettings(FStoreSettings* Settings) const
	{
		check(Settings);

		if (int Value = Parsed["port"].as<int>())
		{
			Settings->StorePort = Value;
		}

		if (int Value = Parsed["recport"].as<int>())
		{
			Settings->RecorderPort = Value;
		}

		if (std::string Value = Parsed["storedir"].as<std::string>(); !Value.empty())
		{
			Settings->StoreDir = Value;
		}

		if (Parsed.count("sponsor-mode"))
		{
			uint32 Value = Parsed["sponsor-mode"].as<uint32>();
			Settings->Sponsored = Value;
		}
	}

	bool GetSponsorPid(uint32& OutSponsorPid) const
	{
		if (uint32 Value = Parsed["sponsor"].as<uint32>())
		{
			OutSponsorPid = Value;
			return true;
		}
		return false;
	}

	bool GetDetach() const
	{
		return Parsed["detach"].count() > 0;
	}

	void PrintHelp() const
	{
		std::string HelpText;
		if (!CurrentCommand)
		{
			HelpText = Options.help({"dummy"});
			HelpText += "Use \"<Command> --help\" to get extended help about specific commands.\n\n";
		}
		else
		{
			std::vector<std::string> HelpGroups;
			for (auto Group : CurrentCommand->HelpGroups)
			{
				HelpGroups.push_back(Group);
			}
			HelpText = Options.help(HelpGroups);
		}

		fputs(HelpText.c_str(), stdout);
	}


private:

	cxxopts::Options Options;
	cxxopts::ParseResult Parsed;

	static FString GetMainHelpString()
	{
		FString Help = "\nUnrealTraceServer acts as a hub between runtimes that are tracing performance "
			"instrumentation and tools like Unreal Insights that consume and present that "
			"data for analysis. TCP ports 1981 and 1989 are used, where the former receives "
			"trace data, and the latter is used by tools to query the server's store.\n\n"
			"UnrealTraceServer will look for a configuration file in \"";

		// Add the settings file path
#if TS_USING(TS_PLATFORM_WINDOWS)
		Help += "%LOCALAPPDATA%/UnrealEngine/Common/UnrealTrace/Settings.ini";
#else
		Help += "~/UnrealEngine/UnrealTrace/Settings.ini";
#endif
		Help += "\". If no file is found default values are used. "
			"Command line arguments takes precedence over values in the configuration file.\n";
			
		return Help;
	}
	
	static struct FCommandHelp
	{
		const char* Command;
		const char* ShortText;
		const char* LongText;
		std::vector<const char*> HelpGroups;
	} CommandHelp[];

	FString ConfigurationHelp;
	const FCommandHelp* CurrentCommand;
};

FOptions::FCommandHelp FOptions::CommandHelp[] = {
	{"fork", "	Starts a background server, upgrading any existing instance. ", 
				"Checks if there is an existing instance running. If the running version is the same "
				"version or newer that instance is used. If a sponsor pid is specified that pid is added "
				"to the running instance.", {"settings", "sponsor"}},
	{"daemon", "The mode that a background server runs in. ", "", {"sponsor"}},
	{"kill", "	Shuts down a currently running instance. ", "", {}},
	{nullptr, nullptr, nullptr, {}}
};

// {{{1 return codes -----------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
enum : int
{
	Result_Ok				= 0,
	Result_BegunCreateFail,
	Result_BegunExists,
	Result_BegunTimeout,
	Result_CopyFail,
	Result_ForkFail,
	Result_LaunchFail,
	Result_NoQuitEvent,
	Result_ProcessOpenFail,
	Result_QuitExists,
	Result_RenameFail,
	Result_SharedMemFail,
	Result_SharedMemTruncFail,
	Result_OpenFailPid,
	Result_ReadFailPid,
	Result_LockClaimFail,
	Result_UnexpectedError,
	Result_InvalidArgError,
	Result_SponsorAddFail,
};



// {{{1 windows ----------------------------------------------------------------

#if TS_USING(TS_PLATFORM_WINDOWS)
////////////////////////////////////////////////////////////////////////////////
class FWinHandle
{
public:
	FWinHandle(HANDLE InHandle)
	: Handle(InHandle)
	{
		if (Handle == INVALID_HANDLE_VALUE)
		{
			Handle = nullptr;
		}
	}

				~FWinHandle()				{ if (Handle) CloseHandle(Handle); }
				operator HANDLE () const	{ return Handle; }
	bool		IsValid() const				{ return Handle != nullptr; }

private:
	HANDLE		Handle;
	FWinHandle&	operator = (const FWinHandle&) = delete;
				FWinHandle(const FWinHandle&) = delete;
				FWinHandle(const FWinHandle&&) = delete;
};


////////////////////////////////////////////////////////////////////////////////
static bool IsAllowedToBreakAway()
{
	JOBOBJECT_BASIC_LIMIT_INFORMATION JobLimits;
	if (QueryInformationJobObject(NULL, JobObjectBasicLimitInformation, &JobLimits, sizeof(JOBOBJECT_BASIC_LIMIT_INFORMATION), NULL))
	{
		return !!(JobLimits.LimitFlags & JOB_OBJECT_LIMIT_BREAKAWAY_OK);
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////
static bool LaunchUnfancy(const wchar_t* Binary, wchar_t* CommandLine)
{
	/* No frills spawning of a child process */

	uint32 CreateProcFlags = 0;
	if (IsAllowedToBreakAway())
	{
		CreateProcFlags |= CREATE_BREAKAWAY_FROM_JOB;
	}
	STARTUPINFOW StartupInfo = { sizeof(STARTUPINFOW) };
	PROCESS_INFORMATION ProcessInfo = {};

	BOOL bOk = CreateProcessW(Binary, CommandLine, nullptr, nullptr, FALSE,
		CreateProcFlags, nullptr, nullptr, &StartupInfo, &ProcessInfo);

	if (bOk == FALSE)
	{
		return false;
	}

	CloseHandle(ProcessInfo.hProcess);
	CloseHandle(ProcessInfo.hThread);
	return true;
}

////////////////////////////////////////////////////////////////////////////////
static bool LaunchUnprivileged(const wchar_t* Binary, wchar_t* CommandLine)
{
	/* Ordinary child process without inheriting any administrator rights */
	uint32 CreateProcFlags = 0;
	if (IsAllowedToBreakAway())
	{
		CreateProcFlags |= CREATE_BREAKAWAY_FROM_JOB;
	}
	STARTUPINFOW StartupInfo = { sizeof(STARTUPINFOW) };
	PROCESS_INFORMATION ProcessInfo = {};

	SAFER_LEVEL_HANDLE SafeLevel = nullptr;
	BOOL bOk = SaferCreateLevel(SAFER_SCOPEID_USER,
		SAFER_LEVELID_NORMALUSER, SAFER_LEVEL_OPEN, &SafeLevel, nullptr);
	if (bOk == FALSE)
	{
		TS_LOG("Unable to create safe level to launch (error %0xd)", GetLastError());
		return false;
	}

	HANDLE AccessToken;
	if (SaferComputeTokenFromLevel(SafeLevel, nullptr, &AccessToken, 0, nullptr))
	{
		bOk = CreateProcessAsUserW(AccessToken, Binary, CommandLine,
			nullptr, nullptr, FALSE, CreateProcFlags, nullptr, nullptr,
			&StartupInfo, &ProcessInfo);

		if (!bOk)
		{
			TS_LOG("Failed to create process as user (error %0xd)", GetLastError());
		}

		CloseHandle(AccessToken);
	}
	else {
		TS_LOG("Unable to create safe level token to launch (error %0xd)", GetLastError());
	}

	SaferCloseLevel(SafeLevel);
	return (bOk == TRUE);
}

////////////////////////////////////////////////////////////////////////////////
static bool LaunchUnelevated(const wchar_t* Binary, wchar_t* CommandLine)
{
	/* Launches a binary with the shell as its parent. The shell (such as
	   Explorer) should be an unelevated process. */

	// No sense in using this route if we are not elevated in the first place
	if (IsUserAnAdmin() == FALSE)
	{
		TS_LOG("User is admin, switching to LaunchUnpriviledged");
		return LaunchUnprivileged(Binary, CommandLine);
	}

	// Get the users' shell process and open it for process creation
	HWND ShellWnd = GetShellWindow();
	if (ShellWnd == nullptr)
	{
		TS_LOG("Cannot get shell!");
		return false;
	}

	DWORD ShellPid;
	GetWindowThreadProcessId(ShellWnd, &ShellPid);

	FWinHandle Process = OpenProcess(PROCESS_CREATE_PROCESS, FALSE, ShellPid);
	if (!Process.IsValid())
	{
		TS_LOG("Cannot open shell!");
		return false;
	}

	// Creating a process as a child of another process is done by setting a
	// thread-attribute list on the startup info passed to CreateProcess()
	SIZE_T AttrListSize;
	InitializeProcThreadAttributeList(nullptr, 1, 0, &AttrListSize);

	auto AttrList = (PPROC_THREAD_ATTRIBUTE_LIST)malloc(AttrListSize);
	OnScopeExit([&] () { free(AttrList); });

	if (!InitializeProcThreadAttributeList(AttrList, 1, 0, &AttrListSize))
	{
		return false;
	}

	BOOL bOk = UpdateProcThreadAttribute(AttrList, 0, PROC_THREAD_ATTRIBUTE_PARENT_PROCESS,
		(HANDLE*)&Process, sizeof(Process), nullptr, nullptr);
	if (!bOk)
	{
		TS_LOG("Unable to update thread attributes");
		return false;
	}

	// By this point we know we are an elevated process. It is not allowed to
	// create a process as a child of another unelevated process that share our
	// elevated console window if we have one. So we'll need to create a new one.
	uint32 CreateProcFlags = EXTENDED_STARTUPINFO_PRESENT;
	if (IsAllowedToBreakAway())
	{
		CreateProcFlags |= CREATE_BREAKAWAY_FROM_JOB;
	}
	if (GetConsoleWindow() != nullptr)
	{
		CreateProcFlags |= CREATE_NEW_CONSOLE;
	}
	else
	{
		CreateProcFlags |= DETACHED_PROCESS;
	}

	// Everything is set up now so we can proceed and launch the process
	STARTUPINFOEXW StartupInfo = { sizeof(STARTUPINFOEXW) };
	StartupInfo.lpAttributeList = AttrList;
	PROCESS_INFORMATION ProcessInfo = {};

	bOk = CreateProcessW(Binary, CommandLine, nullptr, nullptr, FALSE,
		CreateProcFlags, nullptr, nullptr, &StartupInfo.StartupInfo, &ProcessInfo);
	if (bOk == FALSE)
	{
		TS_LOG("Create process failed with error %0xd", GetLastError());
		return false;
	}

	CloseHandle(ProcessInfo.hProcess);
	CloseHandle(ProcessInfo.hThread);
	return true;
}



////////////////////////////////////////////////////////////////////////////////
static const wchar_t*	GIpcName					= L"Local\\UnrealTraceInstance";
static const int32		GIpcSize					= 4 << 10;
const wchar_t*			GQuitEventName				= L"Local\\UnrealTraceEvent";
static const wchar_t*	GBegunEventName				= L"Local\\UnrealTraceEventBegun";
static int				MainDaemon(int, char**, const FOptions&);
void					AddToSystemTray(FStoreService&);
void					RemoveFromSystemTray();

////////////////////////////////////////////////////////////////////////////////
static int CreateExitCode(uint32 Id)
{
	return (GetLastError() & 0xfff) | (Id << 12);
}

////////////////////////////////////////////////////////////////////////////////
static int MainKillImpl(int ArgC, char** ArgV, const FInstanceInfo* InstanceInfo)
{
	// Signal to the existing instance to shutdown or forcefully do it if it
	// does not respond in time.

	TS_LOG("Opening quit event");
	FWinHandle QuitEvent = CreateEventW(nullptr, TRUE, FALSE, GQuitEventName);
	if (GetLastError() != ERROR_ALREADY_EXISTS)
	{
		TS_LOG("Not found (gle=%d)", GetLastError());
		return Result_NoQuitEvent;
	}

	TS_LOG("Open the process %d", InstanceInfo->Pid);
	DWORD OpenProcessFlags = PROCESS_TERMINATE | SYNCHRONIZE;
	FWinHandle ProcHandle = OpenProcess(OpenProcessFlags, FALSE, InstanceInfo->Pid);
	if (!ProcHandle.IsValid())
	{
		TS_LOG("Unsuccessful (gle=%d)", GetLastError());
		return CreateExitCode(Result_ProcessOpenFail);
	}

	TS_LOG("Firing quit event and waiting for process");
	SetEvent(QuitEvent);

	if (WaitForSingleObject(ProcHandle, 5000) == WAIT_TIMEOUT)
	{
		TS_LOG("Timeout. Force terminating");
		TerminateProcess(ProcHandle, 10);
	}

	return Result_Ok;
}

////////////////////////////////////////////////////////////////////////////////
static int MainKill(int ArgC, char** ArgV, const FOptions& Options)
{
	// Find if an existing instance is already running.
	FWinHandle IpcHandle = OpenFileMappingW(FILE_MAP_ALL_ACCESS, false, GIpcName);
	if (!IpcHandle.IsValid())
	{
		TS_LOG("All good. There was no active UTS process");
		return Result_Ok;
	}

	// There is an instance running so we can get its info block
	void* IpcPtr = MapViewOfFile(IpcHandle, FILE_MAP_ALL_ACCESS, 0, 0, GIpcSize);
	FMmapScope MmapScope(IpcPtr);
	const auto* InstanceInfo = MmapScope.As<const FInstanceInfo>();
	InstanceInfo->WaitForReady();

	return MainKillImpl(ArgC, ArgV, InstanceInfo);
}

////////////////////////////////////////////////////////////////////////////////
static int MainFork(int ArgC, char** ArgV, const FOptions& Options)
{
	// Check for an existing instance that is already running.
	TS_LOG("Opening exist instance's shared memory");
	FWinHandle IpcHandle = OpenFileMappingW(FILE_MAP_ALL_ACCESS, false, GIpcName);
	if (IpcHandle.IsValid())
	{
		void* IpcPtr = MapViewOfFile(IpcHandle, FILE_MAP_ALL_ACCESS, 0, 0, GIpcSize);
		FMmapScope MmapScope(IpcPtr);

		auto* InstanceInfo = MmapScope.As<FInstanceInfo>();
		InstanceInfo->WaitForReady();
#if TS_USING(TS_BUILD_DEBUG)
		if (false)
#else
		if (!InstanceInfo->IsOlder())
#endif
		{
			TS_LOG("Existing instance is the same age or newer");

			// If a parent pid was specified, add to the list of sponsor processes.
			if (uint32 PidToAdd = 0; Options.GetSponsorPid(PidToAdd))
			{
				bool bSuccess = false;
				while (!bSuccess)
				{
					for (auto& Pid : InstanceInfo->SponsorPids)
					{
						uint32 Expected = 0;
						if (Pid.compare_exchange_strong(Expected, PidToAdd))
						{
							bSuccess = true;
							break;
						}
					}
					if (!bSuccess)
					{
						TS_LOG("Sponsor slots full, relax a second...");
						Sleep(1000);
					}
				}

				if (!bSuccess)
				{
					TS_LOG("Failed to add sponsor process in existing instance.");
					return Result_SponsorAddFail;
				}
			}

			return Result_Ok;
		}

		// Kill the other instance.
		int KillRet = MainKillImpl(0, nullptr, InstanceInfo);
		if (KillRet == Result_NoQuitEvent)
		{
			// If no quit event was found then we shall assume that another new
			// store instance beat us to it.
			TS_LOG("Looks like someone else has already taken care of the upgrade");
			return Result_Ok;
		}

		if (KillRet != Result_Ok)
		{
			TS_LOG("Kill attempt failed (ret=%d)", KillRet);
			return KillRet;
		}
	}
	else
	{
		TS_LOG("No existing process/shared memory found");
	}

	// Get this binary's path
	TS_LOG("Getting binary path");
	wchar_t BinPath[MAX_PATH];
	uint32 BinPathLen = GetModuleFileNameW(nullptr, BinPath, TS_ARRAY_COUNT(BinPath));
	if (GetLastError() == ERROR_INSUFFICIENT_BUFFER)
	{
		// This should never really happen...
		TS_LOG("MAX_PATH is not enough");
		return CreateExitCode(Result_UnexpectedError);
	}
	TS_LOG("Binary located at '%ls'", BinPath);

	// Calculate where to store the binaries.
	FPath DestPath;
	GetUnrealTraceHome(DestPath);
	{
		wchar_t Buffer[64];
		_snwprintf(Buffer, TS_ARRAY_COUNT(Buffer), L"Bin/%08x/UnrealTraceServer.exe", FInstanceInfo::CurrentVersion);
		DestPath /= Buffer;
	}
	TS_LOG("Run path '%ls'", DestPath.c_str());

#if TS_USING(TS_BUILD_DEBUG)
	// Debug builds will always do the copy.
	{
		std::error_code ErrorCode;
		std::filesystem::remove(DestPath, ErrorCode);
	}
#endif

	// Copy the binary out to a location where it doesn't matter if the file
	// gets locked by the OS.
	if (!std::filesystem::is_regular_file(DestPath))
	{
		TS_LOG("Copying to run path");

		std::error_code ErrorCode;
		std::filesystem::create_directories(DestPath.parent_path(), ErrorCode);

		// Tag the destination with our PID and copy
		DWORD OurPid = GetCurrentProcessId();
		wchar_t Buffer[16];
		_snwprintf(Buffer, TS_ARRAY_COUNT(Buffer), L"_%08x", OurPid);
		FPath TempPath = DestPath;
		TempPath += Buffer;
		if (!std::filesystem::copy_file(BinPath, TempPath))
		{
			TS_LOG("File copy failed (gle=%d)", GetLastError());
			return CreateExitCode(Result_CopyFail);
		}

		// Move the file into place. If this fails because the file exists then
		// another instance has beaten us to the punch.
		std::filesystem::rename(TempPath, DestPath, ErrorCode);
		if (ErrorCode)
		{
			bool bRaceLost = (ErrorCode == std::errc::file_exists);
			TS_LOG("Rename to destination failed (bRaceLost=%c)", bRaceLost);
			return bRaceLost ? Result_Ok : CreateExitCode(Result_RenameFail);
		}
	}
	else
	{
		TS_LOG("Already exists");
	}

	// Launch a new instance as a daemon and wait until we know it has started
	TS_LOG("Creating begun event");
	FWinHandle BegunEvent = CreateEventW(nullptr, TRUE, FALSE, GBegunEventName);
	if (GetLastError() == ERROR_ALREADY_EXISTS)
	{
		TS_LOG("Did not work (gle=%d)", GetLastError());
		return CreateExitCode(Result_BegunExists);
	}

	// For debugging ease and consistency we will daemonize in this process
	// instead of spawning a second one.
#if TS_USING(TS_DAEMON_THREAD)
	std::thread DaemonThread([=] () { MainDaemon(ArgC, ArgV, Options); });
#else
	std::wstring CommandLine = L"UnrealTraceServer.exe daemon -d ";

	auto ContainsSpace = [](LPCWSTR Arg) -> bool {
		for (uint32 c = 0, Len = (uint32)wcslen(Arg); c < Len; ++c)
		{
			if (iswspace(Arg[c]))
			{
				return true;
			}
		}
		return false;
	};

	for (int i = 1; i < ArgC; ++i)
	{
		FWinApiStr Arg(ArgV[i]);
		const bool bContainsSpace = ContainsSpace(Arg);
		CommandLine += L" ";
		if (bContainsSpace) CommandLine += L"\"";
		CommandLine += FWinApiStr(ArgV[i]);
		if (bContainsSpace) CommandLine += L"\"";
	}
	if (!LaunchUnelevated(DestPath.c_str(), CommandLine.data()))
	{
		TS_LOG("Unelevated launch failed (gle=%d)", GetLastError());
		if (!LaunchUnprivileged(DestPath.c_str(), CommandLine.data()))
		{
			TS_LOG("Unprivileged launch failed (gle=%d)", GetLastError());
			if (!LaunchUnfancy(DestPath.c_str(), CommandLine.data()))
			{
				TS_LOG("Launch failed (gle=%d)", GetLastError());
				return CreateExitCode(Result_LaunchFail);
			}
		}
	}
#endif // !TS_BUILD_DEBUG

	TS_LOG("Waiting on begun event");
	int Ret = Result_Ok;
	if (WaitForSingleObject(BegunEvent, 5000) == WAIT_TIMEOUT)
	{
		TS_LOG("Wait timed out (gle=%d)", GetLastError());
		Ret = CreateExitCode(Result_BegunTimeout);
	}

#if TS_USING(TS_DAEMON_THREAD)
	static bool volatile bShouldExit = false;
	while (true)
	{
		Sleep(500);

		if (bShouldExit)
		{
			FWinHandle QuitEvent = CreateEventW(nullptr, TRUE, FALSE, GQuitEventName);
			SetEvent(QuitEvent);
			break;
		}
	}

	DaemonThread.join();
#endif

	TS_LOG("Complete (ret=%d)", Ret);
	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
static int MainDaemon(int ArgC, char** ArgV, const FOptions& Options)
{
	// Move the working directory to be where this binary is located.
	TS_LOG("Setting working directory");
	wchar_t BinPath[MAX_PATH];
	uint32 BinPathLen = GetModuleFileNameW(nullptr, BinPath, TS_ARRAY_COUNT(BinPath));
	if (GetLastError() != ERROR_INSUFFICIENT_BUFFER && BinPathLen > 0)
	{
		std::error_code ErrorCode;
		FPath BinDir(BinPath);
		BinDir = BinDir.parent_path();
		std::filesystem::current_path(BinDir, ErrorCode);
		const char* Result = ErrorCode ? "Failed" : "Succeeded";
		TS_LOG("%s setting '%ls' (gle=%d)", Result, BinDir.c_str(), GetLastError());
	}
	else
	{
		TS_LOG("Something went wrong (gle=%d)", GetLastError());
	}

	// Create a piece of shared memory so all store instances can communicate.
	TS_LOG("Creating some shared memory");
	FWinHandle IpcHandle = CreateFileMappingW(INVALID_HANDLE_VALUE, nullptr,
		PAGE_READWRITE, 0, GIpcSize, GIpcName);
	if (!IpcHandle.IsValid())
	{
		TS_LOG("Creation unsuccessful (gle=%d)", GetLastError());
		return CreateExitCode(Result_SharedMemFail);
	}

	// Create a named event so others can tell us to quit.
	TS_LOG("Creating a quit event");
	FWinHandle QuitEvent = CreateEventW(nullptr, TRUE, FALSE, GQuitEventName);
	if (GetLastError() == ERROR_ALREADY_EXISTS)
	{
		// This really should not happen. It is expected that only one process
		// will get this far (gated by the shared-memory object creation).
		TS_LOG("It unexpectedly exists already");
		return CreateExitCode(Result_QuitExists);
	}

	// Fill out the Ipc details and publish.
	void* IpcPtr = MapViewOfFile(IpcHandle, FILE_MAP_ALL_ACCESS, 0, 0, GIpcSize);
	
	TS_LOG("Writing shared instance info");
	FMmapScope MmapScope(IpcPtr);
	auto* InstanceInfo = MmapScope.As<FInstanceInfo>();
	InstanceInfo->Set();
	

	// Set locale for std function application wide.
	std::setlocale(LC_ALL, ".UTF-8");
	
	TS_LOG("Starting the store");
	FPath HomeDir;
	GetUnrealTraceHome(HomeDir);

	// Read settings from configuration file
	FStoreSettings* Settings = new FStoreSettings();
	Settings->ReadFromSettings(HomeDir);
	// Override with command line arguments.
	Options.ApplyToSettings(Settings);
	// Display final settings to user
	Settings->PrintToLog();

	// Read other arguments
	uint32 ParentPid = 0;
	if (Settings->Sponsored && !Options.GetSponsorPid(ParentPid))
	{
		TS_LOG("Error: Deamon is configured to run in sponsored mode, but no sponsor pid has been specified.");
		return Result_InvalidArgError;
	}
	InstanceInfo->AddSponsor(ParentPid);

	// Fire up the store
	FStoreService* StoreService = FStoreService::Create(Settings, InstanceInfo);
	OnScopeExit([StoreService]() { delete StoreService; });

	if (Options.GetDetach())
	{
		TS_LOG("Detaching from console now.");
		FreeConsole();
	}

	// Let every one know we've started.
	{
		FWinHandle BegunEvent = CreateEventW(nullptr, TRUE, FALSE, GBegunEventName);
		if (BegunEvent.IsValid())
		{
			SetEvent(BegunEvent);
		}
	}

	// To clearly indicate to users that we are around we'll add an icon to the
	// system tray.
	AddToSystemTray(*StoreService);

	// Wait to be told to resign.
	WaitForSingleObject(QuitEvent, INFINITE);

	// Clean up. We are done here.
	RemoveFromSystemTray();

	TS_LOG("Daemon is exiting without errors.");
	return Result_Ok;
}

////////////////////////////////////////////////////////////////////////////////
int WINAPI WinMain(HINSTANCE, HINSTANCE, LPSTR, int)
{
	// Get command line arguments and convert to UTF8
	const wchar_t* CommandLine = GetCommandLineW();
	int ArgC;
	wchar_t** WideArgV = CommandLineToArgvW(CommandLine, &ArgC);

	TInlineBuffer<320> ArgBuffer;
	char** ArgV = (char**)(ArgBuffer->Append(ArgC * sizeof(char*)));
	for (int i = 0; i < ArgC; ++i)
	{
		const wchar_t* WideArg = WideArgV[i];
		int32 ArgSize = WideCharToMultiByte(CP_UTF8, 0, WideArg, -1, nullptr, 0, nullptr, nullptr);

		char* Arg = (char*)(ArgBuffer->Append(ArgSize));
		WideCharToMultiByte(CP_UTF8, 0, WideArg, -1, Arg, ArgSize, nullptr, nullptr);

		ArgV[i] = Arg;
	}

	// The proper entry point
	extern int main(int, char**);
	int Ret = main(ArgC, ArgV);

	// Clean up
	LocalFree(ArgV);
	return Ret;
}

#endif // TS_PLATFORM_WINDOWS



// {{{1 linux/mac --------------------------------------------------------------

#if TS_USING(TS_PLATFORM_LINUX) || TS_USING(TS_PLATFORM_MAC)

////////////////////////////////////////////////////////////////////////////////
static int MainDaemonImpl(int, char**, pid_t, const FOptions& Options);

////////////////////////////////////////////////////////////////////////////////
#if TS_USING(TS_LEGACY_LOCK_FILE)

static int MainKillImpl(int ArgC, char** ArgV, pid_t DaemonPid);

static FPath GetLockFilePath()
{
	return "/tmp/UnrealTraceServer.pid";
}

static int LegacyLockFile()
{
	// Open the pid file to detect an existing instance
	FPath DotPidPath = GetLockFilePath();
	TS_LOG("Checking for a '%s' lock file", DotPidPath.c_str());
	for (int DotPidFd = open(DotPidPath.c_str(), O_RDONLY); DotPidFd >= 0; )
	{
		OnScopeExit([DotPidFd] () { close(DotPidFd); });

		// If we can claim a write lock then the lock file is orphaned
		struct flock FileLock = { .l_type = F_WRLCK };
		int Result = fcntl(DotPidFd, F_GETLK, &FileLock);
		if (Result != 0 || FileLock.l_type == F_UNLCK)
		{
			TS_LOG("Lock file appears to be orphaned. Unlinking it");
			std::error_code ErrorCode;
			std::filesystem::remove(DotPidPath, ErrorCode);
			break;
		}

		// An instance running with pid file is by definition older than us
		TS_LOG("Killing an older instance that is already running");
		int KillRet = MainKillImpl(0, nullptr, FileLock.l_pid);
		if (KillRet == Result_NoQuitEvent)
		{
			// If no quit event was found then we shall assume that another new
			// store instance beat us to it.
			TS_LOG("Looks like someone else has already taken care of the upgrade");
			return Result_Ok;
		}

		if (KillRet != Result_Ok)
		{
			TS_LOG("Kill attempt failed (ret=%d)", KillRet);
			return KillRet;
		}
	}
	return Result_Ok;
}
#endif

////////////////////////////////////////////////////////////////////////////////
static int MainKillImpl(int ArgC, char** ArgV, pid_t DaemonPid)
{
	if (DaemonPid == 0)
	{
		return Result_NoQuitEvent;
	}

	// Issue the terminate signal
	TS_LOG("Sending SIGTERM to %d", DaemonPid);
	if (kill(DaemonPid, SIGTERM) < 0)
	{
		TS_LOG("Failed to send SIGTERM");
		return Result_SharedMemFail;
	}

	// Wait for the process to end. If it takes too long, kill it.
	TS_LOG("Waiting for pid %d", DaemonPid);
	const uint32 SleepMs = 47;
	timespec SleepTime = { 0, SleepMs * 1000 * 1000 };
	nanosleep(&SleepTime, nullptr);
	for (uint32 i = 0; ; i += SleepMs)
	{
		if (i >= 5000) // "5000" for grep-ability
		{
			kill(DaemonPid, SIGKILL);
			TS_LOG("Timed out. Sent SIGKILL instead (errno=%d)", errno);
			break;
		}

		if (kill(DaemonPid, 0) < 0)
		{
			if (errno == ESRCH)
			{
				TS_LOG("Process no longer exists");
				break;
			}
		}

		nanosleep(&SleepTime, nullptr);
	}

	return Result_Ok;
}

////////////////////////////////////////////////////////////////////////////////
// macOS shm_open rejects O_CLOEXEC (EINVAL); set it via fcntl after open instead.
#if TS_USING(TS_PLATFORM_MAC)
static int SharedMemoryOpen(const char* Name, int Flags, mode_t Mode)
{
	int Fd = shm_open(Name, Flags & ~O_CLOEXEC, Mode);
	if (Fd >= 0 && (Flags & O_CLOEXEC))
		fcntl(Fd, F_SETFD, FD_CLOEXEC);
	return Fd;
}
#else
#define SharedMemoryOpen shm_open
#endif

////////////////////////////////////////////////////////////////////////////////
static int MainKill(int ArgC, char** ArgV, const FOptions& Options)
{
	TS_LOG("Opening shared memory");
	int Fd = SharedMemoryOpen("/UnrealTraceServer", O_RDONLY | O_CLOEXEC, 0666);
	if (Fd > 0)
	{
		void* BufferPtr = mmap(nullptr, sizeof(FInstanceInfo), PROT_READ, MAP_SHARED, Fd, 0);
		if (!BufferPtr)
		{
			TS_LOG("Unable to map shared memory");
			return Result_SharedMemFail;
		}

		FMmapScope Buffer(BufferPtr);
		FInstanceInfo* InstanceInfo = Buffer.As<FInstanceInfo>();
		
		int KillRet = MainKillImpl(0, nullptr, InstanceInfo->Pid);
		if (KillRet == Result_NoQuitEvent)
		{
			TS_LOG("Looks like someone else has already taken care of stopping");
			return Result_Ok;
		}

		if (KillRet != Result_Ok)
		{
			TS_LOG("Kill attempt failed (ret=%d)", KillRet);
			return KillRet;
		}
	}
#if TS_USING(TS_LEGACY_LOCK_FILE)
	else
	{
		TS_LOG("Shared memory doesn't exist, checking legacy lock file");
		return LegacyLockFile(); 
	}
#endif

	return Result_Ok;
}


////////////////////////////////////////////////////////////////////////////////
static int MainFork(int ArgC, char** ArgV, const FOptions& Options)
{
	TS_LOG("Opening shared memory");
	int Fd = SharedMemoryOpen("/UnrealTraceServer", O_RDWR | O_CLOEXEC, 0666);
	if (Fd > 0)
	{
		void* BufferPtr = mmap(nullptr, sizeof(FInstanceInfo), PROT_READ | PROT_WRITE, MAP_SHARED, Fd, 0);
		if (!BufferPtr)
		{
			TS_LOG("Unable to map shared memory");
			return Result_SharedMemFail;
		}

		FMmapScope Buffer(BufferPtr);
		FInstanceInfo* InstanceInfo = Buffer.As<FInstanceInfo>();

		// Old enough for this fine establishment?
		bool bExists = kill(InstanceInfo->Pid, 0) == 0;
		if (bExists && !InstanceInfo->IsOlder())
		{
			TS_LOG("Existing instance is the same age or newer");

			if (uint32 PidToAdd = 0; Options.GetSponsorPid(PidToAdd))
			{
				bool bSuccess = false;
				while (!bSuccess)
				{
					for (auto& Pid : InstanceInfo->SponsorPids)
					{
						uint32 Expected = 0;
						if (Pid.compare_exchange_strong(Expected, PidToAdd))
						{
							bSuccess = true;
							break;
						}
					}
					if (!bSuccess)
					{
						TS_LOG("Sponsor slots full, relax a second...");
						sleep(1);
					}
				}

				if (!bSuccess)
				{
					TS_LOG("Failed to add sponsor process in existing instance.");
					return Result_SponsorAddFail;
				}
			}

			return Result_Ok;
		}

		// If we've got this far then there's an instance running that is old
		if (bExists)
		{
			TS_LOG("Killing an older instance (pid %u) that is already running", InstanceInfo->Pid);
			int KillRet = MainKillImpl(0, nullptr, InstanceInfo->Pid);
			if (KillRet == Result_NoQuitEvent)
			{
				// If no quit event was found then we shall assume that another new
				// store instance beat us to it.
				TS_LOG("Looks like someone else has already taken care of the upgrade");
				return Result_Ok;
			}

			if (KillRet != Result_Ok)
			{
				TS_LOG("Kill attempt failed (ret=%d)", KillRet);
				return KillRet;
			}
		}
	}
#if TS_USING(TS_LEGACY_LOCK_FILE)
	else
	{
		TS_LOG("Shared memory doesn't exist, checking legacy lock file");
		if (int Result = LegacyLockFile(); Result != Result_Ok)
		{
			return Result;
		}
	}
#endif

	// Launch a daemonized version of ourselves. For debugging ease and
	// consistency we will daemonize in this process instead of spawning
	// a second one.
	pid_t DaemonPid = -1;
#if TS_USING(TS_DAEMON_THREAD)
	std::thread DaemonThread([=] () { MainDaemonImpl(ArgC, ArgV, 0, Options); });
#else
	TS_LOG("Forking process");
	pid_t ParentPid = getpid();
	DaemonPid = fork();
	if (DaemonPid < 0)
	{
		TS_LOG("Failed (errno=%d)", errno);
		return Result_ForkFail;
	}
	else if (DaemonPid == 0)
	{
		return MainDaemonImpl(ArgC, ArgV, ParentPid, Options);
	}
#endif // TS_BUILD_DEBUG

	// Wait for the daemon to indicate that it has started the store.
	int Ret = Result_Ok;
#if TS_USING(TS_DAEMON_THREAD)
	DaemonThread.join();
#else
	TS_LOG("Wait until we know the daemon has started.");

	// Create another fork to act as a timeout.
	pid_t TimeoutPid = fork();
	if (TimeoutPid == 0)
	{
		timespec TimeoutTime = { 5, 0 };
		nanosleep(&TimeoutTime, nullptr);
		exit(0);
	}

	struct sigaction SigAction = {};
	SigAction.sa_flags = int(SA_RESETHAND);
	SigAction.sa_handler = [] (int Signal) {
		TS_LOG("Parent received signal %d", Signal);
	};
	sigaction(SIGUSR1, &SigAction, nullptr);

	// Now we have two children (daemon and timeout) and a signal to wait for
	int WaitResult = -1;
	pid_t WaitedPid = waitpid(0, &WaitResult, 0);
	if (WaitedPid == TimeoutPid)
	{
		TS_LOG("Timed out");
		Ret = Result_BegunTimeout;
	}
	else if (WaitedPid == DaemonPid)
	{
		TS_LOG("Daemon exited unexpectedly (wait_result=%d)", WaitResult);
		Ret = WEXITSTATUS(WaitResult);
	}
	else
	{
		TS_LOG("Daemon signalled successful start");
	}
	kill(TimeoutPid, SIGKILL);
#endif // debug

	TS_LOG("Forked complete (ret=%d)", Ret);
	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
static int MainDaemonImpl(int ArgC, char** ArgV, pid_t ParentPid, const FOptions& Options)
{
	TS_LOG("Opening shared memory");
	int Fd = SharedMemoryOpen("/UnrealTraceServer", O_RDWR | O_CREAT | O_CLOEXEC, 0666);
	if (Fd < 0)
	{
		TS_LOG("Unable to create shared memory: %u", errno);
		return Result_SharedMemFail;
	}
	OnScopeExit([]() { shm_unlink("/UnrealTraceServer"); });

	// Set the size of the file. Note that MacOS allows only setting the size only once.
#if TS_USING(TS_PLATFORM_MAC)
	struct stat Stat;
	if (fstat(Fd,&Stat) == 0)
	{
		if (Stat.st_size == 0)
		{
			if(ftruncate(Fd, sizeof(FInstanceInfo)) != 0)
			{
				TS_LOG("Unable to size shared memory: %d", errno);
				return Result_SharedMemFail;
			}
		}
		else 
		{
			TS_LOG("Shared memory sized: %u", Stat.st_size);
		}
	}
	else 
	{
		TS_LOG("Cannot read shared memory stats: %d", errno);
	}
#else
	if (ftruncate(Fd, sizeof(FInstanceInfo)))
	{
		TS_LOG("Unable to size shared memory: %d", errno);
		return Result_SharedMemFail;
	}
#endif

	void* BufferPtr = mmap(nullptr, sizeof(FInstanceInfo), PROT_READ | PROT_WRITE, MAP_SHARED, Fd, 0);
	if (!BufferPtr)
	{
		TS_LOG("Unable to map shared memory");
		return Result_SharedMemFail;
	}

	FMmapScope Buffer(BufferPtr);
	FInstanceInfo* InstanceInfo = Buffer.As<FInstanceInfo>();

	// Fire up the store
	TS_LOG("Starting the store");
	FStoreSettings* Settings = new FStoreSettings();
	FStoreService* StoreService = nullptr;
	
	FPath HomeDir;
	GetUnrealTraceHome(HomeDir);

	// Read settings from configuration file
	Settings->ReadFromSettings(HomeDir);

	// Override with command line arguments
	Options.ApplyToSettings(Settings);
	// Display final settings to user
	Settings->PrintToLog();

	// Get sponsor pid
	uint32 SponsorPid = 0;
	if (Settings->Sponsored && !Options.GetSponsorPid(SponsorPid))
	{
		TS_LOG("Error: Deamon is configured to run in sponsored mode, but no sponsor pid has been specified.");
		return Result_InvalidArgError;
	}
	// Add given sponsor pid regardless, in case we suddenly enable
	// sponsor mode 
	InstanceInfo->AddSponsor(SponsorPid);

	StoreService = FStoreService::Create(Settings, InstanceInfo);
	OnScopeExit([StoreService] () { delete StoreService; });

	// Fill out the shared memory with details about this instance
	InstanceInfo->Set();

	// Detach from our parent and from any controlling terminal
	for (int Result = setsid(); Result < 0;)
	{
		if (errno == EPERM)
		{
			break;
		}

		TS_LOG("Failed creating a new session (errno=%d)", errno);
		return Result_UnexpectedError;
	}

	// Signal to any parent that we're done here.
	if (ParentPid != 0)
	{
		TS_LOG("Signalling parent %d", ParentPid);
		kill(ParentPid, SIGUSR1);
	}

	// Wait to be told to resign.
	struct sigaction SigAction = {};
	SigAction.sa_handler = [] (int) {};
	SigAction.sa_flags = int(SA_RESETHAND),
	sigaction(SIGTERM, &SigAction, nullptr);
	sigaction(SIGINT, &SigAction, nullptr);

	// Create and set a signal handler
	sigset_t SignalSet;
	sigemptyset(&SignalSet);
	sigaddset(&SignalSet, SIGTERM);
	sigaddset(&SignalSet, SIGKILL);
	sigaddset(&SignalSet, SIGINT);

	while (true)
	{
		int Signal = -1;
		int Ret = sigwait(&SignalSet, &Signal);
		if (Ret == 0)
		{
			TS_LOG("Received signal %d", Signal);
			break;
		}
	}
	TS_LOG("Daemon is exiting without errors.");
	return Result_Ok;
}

////////////////////////////////////////////////////////////////////////////////
static int MainDaemon(int ArgC, char** ArgV, const FOptions& Options)
{
	return MainDaemonImpl(ArgC, ArgV, 0, Options);
}

#endif // TS_PLATFORM_LINUX/MAC



// {{{1 main -------------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
int main(int ArgC, char** ArgV)
{
	FOptions Options;
	if (auto ExitReason = Options.Parse(ArgC, ArgV); ExitReason)
	{
		return ExitReason == FOptions::Result_HelpRequested ? Result_Ok : Result_InvalidArgError;
	}

	struct {
		const char*	Verb;
		int			(*Entry)(int, char**, const FOptions&);
	} Dispatches[] = {
		{"fork",		MainFork},
		{"daemon",		MainDaemon},
		{"kill",		MainKill},
	};

	for (const auto& Dispatch : Dispatches)
	{
		if (strcmp(ArgV[1], Dispatch.Verb) == 0)
		{
			FPath LogDir;
			GetUnrealTraceHome(LogDir, true);
			InstallCrashHandler(LogDir);
			FLoggingScope LoggingScope(LogDir);

#ifdef TS_WITH_MUSL
			TS_LOG("Using musl");
#endif
			return (Dispatch.Entry)(ArgC - 1, ArgV + 1, Options);
		}
	}

	printf("Unknown command '%s'\n", ArgV[1]);
	return Result_InvalidArgError;
}

/* vim: set noexpandtab foldlevel=1 : */
