// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// {{{1 compiler ---------------------------------------------------------------
#if defined(_MSC_VER)
#	pragma warning(disable: 4200) // zero-sized arrays are non-standard
#endif



// {{{1 build ------------------------------------------------------------------

#define TS_ON					+
#define TS_OFF					-
#define TS_USING(x)				((1 x 1) == 2)

#define TS_PLATFORM_WINDOWS		TS_OFF
#define TS_PLATFORM_LINUX		TS_OFF
#define TS_PLATFORM_MAC			TS_OFF

#define TS_ARCH_X64				TS_ON
#define TS_ARCH_ARM64			TS_OFF

#if defined(_MSC_VER)
#	undef  TS_PLATFORM_WINDOWS
#	define TS_PLATFORM_WINDOWS	TS_ON
#endif

#if defined(__linux__)
#	undef  TS_PLATFORM_LINUX
#	define TS_PLATFORM_LINUX	TS_ON
#endif

#if defined(__APPLE__)
#	undef  TS_PLATFORM_MAC
#	define TS_PLATFORM_MAC		TS_ON
#	if defined(__arm64__)
#		undef TS_ARCH_X64
#		undef TS_ARCH_ARM64
#		define TS_ARCH_X64		TS_OFF
#		define TS_ARCH_ARM64	TS_ON
#	endif
#endif

#define TS_ARRAY_COUNT(x)		(sizeof(x) / sizeof((x)[0]))

#if !defined(TS_BUILD_DEBUG)
#	define TS_BUILD_DEBUG		TS_OFF
#endif



// {{{1 base -------------------------------------------------------------------

#if TS_USING(TS_PLATFORM_WINDOWS)
#	define _HAS_EXCEPTIONS			0
#	define _CRT_SECURE_NO_WARNINGS	0
#endif

#include <algorithm>
#include <atomic>
#include <charconv>
#include <chrono>
#include <cstdint>
#include <ctime>
#include <filesystem>
#include <string_view>
#include <string>
#include <ios>
#include <sstream>
#include <vector>

#if TS_USING(TS_PLATFORM_WINDOWS)
#	define _WIN32_WINNT			0x0a00
#	define WIN32_LEAN_AND_MEAN
#	define NOMINMAX
#	include <Windows.h>
#	include <winsafer.h>
#	include <Shlobj.h>
#	include <shellapi.h>
#	pragma comment(lib, "Advapi32.lib")
#	pragma comment(lib, "Shell32.lib")
#	pragma comment(lib, "User32.lib")
#endif

#include "Asio.h"

using int8		= int8_t;
using int16		= int16_t;
using int32		= int32_t;
using int64		= int64_t;
using uint8		= uint8_t;
using uint16	= uint16_t;
using uint32	= uint32_t;
using uint64	= uint64_t;



// {{{1 util -------------------------------------------------------------------

#define TS_CONCAT_(a, b)		a##b
#define TS_CONCAT(a, b)			TS_CONCAT_(a, b)

/* vim: set noexpandtab foldlevel=1 : */
