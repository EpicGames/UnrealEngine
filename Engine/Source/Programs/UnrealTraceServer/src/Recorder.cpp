// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "AsioIoable.h"
#include "AsioSocket.h"
#include "Recorder.h"
#include "Store.h"
#include "Logging.h"



////////////////////////////////////////////////////////////////////////////////
class FRecorderRelay
	: public FAsioIoSink
{
public:
						FRecorderRelay(asio::ip::tcp::socket& Socket, FStore& InStore);
	virtual				~FRecorderRelay();
	bool				IsOpen();
	void				Close();
	uint32				GetTraceId() const;
	uint32				GetIpAddress() const;
	uint32				GetControlPort() const;
	const FGuid&		GetSessionGuid() const;
	const FGuid&		GetTraceGuid() const;

private:
	virtual void		OnIoComplete(uint32 Id, int32 Size) override;
	bool				CreateTrace();
	bool				ReadMagic();
	bool				ReadMetadata(int32 Size);
	static const uint32	BufferSize = 256 * 1024;
	FAsioSocket			Input;
	FAsioWriteable*		Output = nullptr;
	FStore&				Store;
	uint8*				PreambleCursor;
	uint32				TraceId = 0;
	uint16				ControlPort = 0;
	FGuid				SessionGuid;
	FGuid				TraceGuid;
	uint8				FillDrainCounter = 1;
	uint32				BufferPolarity = 0;
	uint8*				Buffer[2];
	uint32				FillSize[2] = {};

	enum
	{
		OpMagicRead			= 0x1000,
		OpMetadataRead		= 0x1001,
		OpBuffer0			= 0b00,
		OpBuffer1			= 0b10,
		OpFill				= 0b00,
		OpDrain				= 0b01,
	};

	using MagicType				= uint32;
	using MetadataSizeType		= uint16;
	using VersionType			= struct { uint8 Transport; uint8 Protocol; };

	static_assert(sizeof(VersionType) == 2, "Unexpected struct size");
};

////////////////////////////////////////////////////////////////////////////////
FRecorderRelay::FRecorderRelay(asio::ip::tcp::socket& Socket, FStore& InStore)
	// Trace data is a stream and communication is one way. It is implemented
	// this way to share code between sending trace data over the wire and writing
	// it to a file. Because there's no ping/pong we can end up with a half-open
	// TCP connection if the other end doesn't close its socket. So we'll enable
	// keep-alive on the socket and set a short timeout (default is 2hrs).
	: Input(Socket, FAsioSocket::FKeepAliveConfig {true, 15, 2})
	, Store(InStore)
{
	Buffer[0] = new uint8[BufferSize];
	Buffer[1] = new uint8[BufferSize];

	// Kick things off by reading the magic four bytes at the start of the stream
	// along with an additional two bytes that are likely the metadata size.
	uint32 PreambleReadSize = sizeof(MagicType) + sizeof(MetadataSizeType);
	PreambleCursor = Buffer[0] + PreambleReadSize;
	Input.Read(Buffer[0], PreambleReadSize, this, OpMagicRead);
}

////////////////////////////////////////////////////////////////////////////////
FRecorderRelay::~FRecorderRelay()
{
	check(!Input.IsOpen());
	if (Output != nullptr)
	{
		check(!Output->IsOpen());
		delete Output;
	}

	delete[] Buffer[1];
	delete[] Buffer[0];
}

////////////////////////////////////////////////////////////////////////////////
bool FRecorderRelay::IsOpen()
{
	// Even if the input socket has been closed we should still report ourselves
	// as open if there is a pending write on the output.
	bool Ret = (Output != nullptr && FillDrainCounter < 2);
	Ret |= Input.IsOpen();
	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
void FRecorderRelay::Close()
{
	Input.Close();
	if (Output != nullptr)
	{
		Output->Close();
	}
}

////////////////////////////////////////////////////////////////////////////////
uint32 FRecorderRelay::GetTraceId() const
{
	return TraceId;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FRecorderRelay::GetIpAddress() const
{
	return Input.GetRemoteAddress();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FRecorderRelay::GetControlPort() const
{
	return ControlPort;
}

////////////////////////////////////////////////////////////////////////////////
const FGuid& FRecorderRelay::GetSessionGuid() const
{
	return SessionGuid;
}

////////////////////////////////////////////////////////////////////////////////
const FGuid& FRecorderRelay::GetTraceGuid() const
{
	return TraceGuid;
}

////////////////////////////////////////////////////////////////////////////////
bool FRecorderRelay::CreateTrace()
{
	FStore::FNewTrace Trace = Store.CreateTrace();
	TraceId = Trace.Id;
	Output = Trace.Writeable;
	return (Output != nullptr);
}

////////////////////////////////////////////////////////////////////////////////
bool FRecorderRelay::ReadMagic()
{
	const uint8* Cursor = Buffer[0];

	// Here we'll check the magic four bytes at the start of the stream and create
	// a trace to write into if they are bytes we're expecting.

	// We will only support clients that send the magic. Very early clients did
	// not do this but they were unreleased and should no longer be in use.
	if (Cursor[3] != 'T' || Cursor[2] != 'R' || Cursor[1] != 'C')
	{
		return false;
	}

	// Later clients have a metadata block (TRC2). There's loose support for the
	// future too if need be (TRC[3-9]).
	if (Cursor[0] < '2' || Cursor[0] > '9')
	{
		return false;
	}

	// Concatenate metadata into the buffer, first validating the given size is
	// one that we can handle in a single read.
	uint32 MetadataSize = *(MetadataSizeType*)(Cursor + sizeof(MagicType));
	MetadataSize += sizeof(VersionType);
	if (MetadataSize > BufferSize - uint32(ptrdiff_t(PreambleCursor - Cursor)))
	{
		return false;
	}

	Input.Read(PreambleCursor, MetadataSize, this, OpMetadataRead);

	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FRecorderRelay::ReadMetadata(int32 Size)
{
	// At this point Buffer          [magic][md_size][metadata][t_ver][p_ver]
	// looks like this;              Buffer--------->PreambleCursor--------->
	//                                               |---------Size---------|

	// We want to consume [metadata] so some adjustment is required.
	int32 ReadSize = Size - sizeof(VersionType);
	const uint8* Cursor = PreambleCursor;

	// MetadataFields
	while (ReadSize >= 2)
	{
		struct FMetadataHeader
		{
			uint8	Size;
			uint8	Id;
		};
		const auto& MetadataField = *(FMetadataHeader*)(Cursor);

		Cursor += sizeof(FMetadataHeader);
		ReadSize -= sizeof(FMetadataHeader);

		if (ReadSize < MetadataField.Size)
		{
			return false;
		}

		switch (MetadataField.Id) 
		{
		case 0: /* ControlPortFieldId */ 
			ControlPort = *(const uint16*)Cursor;
			break;
		case 1: /* SessionGuid */ 
			SessionGuid = *(const FGuid*)Cursor;
			break;
		case 2: /* TraceGuid */
			TraceGuid = *(const FGuid*)Cursor;
			break;
		}

		Cursor += MetadataField.Size;
		ReadSize -= MetadataField.Size;
	}

	// There should be no data left to consume if the metadata was well-formed
	if (ReadSize != 0)
	{
		return false;
	}

	// Now we've a full preamble we are ready to write the trace.
	if (!CreateTrace())
	{
		return false;
	}

	// Analysis needs the preamble too.
	uint32 PreambleSize = uint32(ptrdiff_t(PreambleCursor - Buffer[0])) + Size;
	OnIoComplete(OpBuffer0|OpFill, PreambleSize);

	return true;
}

////////////////////////////////////////////////////////////////////////////////
void FRecorderRelay::OnIoComplete(uint32 Id, int32 Size)
{
	if (Size < 0)
	{
		FillDrainCounter += (Output != nullptr);
		Close();
		return;
	}

	// A completed preamble read?
	switch (Id)
	{
	case OpMagicRead:
		if (!ReadMagic())
		{
			Close();
		}
		return;

	case OpMetadataRead:
		if (!ReadMetadata(Size))
		{
			Close();
		}
		return;
	}

	// If we've got to here then a fill or drain op has completed.

	// Cache fill sizes so they can be used when we are ready to issue the drain
	if ((Id & 0b01) == 0)
	{
		uint32 BufferIndex = (Id & 0b10) >> 1;
		FillSize[BufferIndex] = Size;
	}

	// We only dispatch another fill/drain pair once the previous two have
	// completed. This is so we don't overlap fills, drains, or buffer use.
	++FillDrainCounter;
	if (FillDrainCounter < 2)
	{
		return;
	}
	FillDrainCounter = 0;

	// At this point we've one buffer that's been filled and another that has
	// finished being drained. We are free to issue two more concurrent ops
	uint32 BufferIndex = (BufferPolarity != 0);
	Output->Write(Buffer[BufferIndex], FillSize[BufferIndex], this, BufferPolarity|OpDrain);
	BufferPolarity ^= 0b10;
	Input.ReadSome(Buffer[BufferIndex ^ 1], BufferSize, this, BufferPolarity|OpFill);
}



////////////////////////////////////////////////////////////////////////////////
uint32 FRecorder::FSession::GetId() const
{
	return Id;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FRecorder::FSession::GetTraceId() const
{
	return Relay->GetTraceId();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FRecorder::FSession::GetIpAddress() const
{
	return Relay->GetIpAddress();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FRecorder::FSession::GetControlPort() const
{
	return Relay->GetControlPort();
}

////////////////////////////////////////////////////////////////////////////////
const FGuid& FRecorder::FSession::GetSessionGuid() const
{
	return Relay->GetSessionGuid();
}

////////////////////////////////////////////////////////////////////////////////
const FGuid& FRecorder::FSession::GetTraceGuid() const
{
	return Relay->GetTraceGuid();
}



////////////////////////////////////////////////////////////////////////////////
FRecorder::FRecorder(asio::io_context& IoContext, FStore& InStore)
: FAsioTcpServer(IoContext)
, FAsioTickable(IoContext)
, Store(InStore)
{
	StartTick(500);
}

////////////////////////////////////////////////////////////////////////////////
FRecorder::~FRecorder()
{
	check(!FAsioTickable::IsActive());
	check(!FAsioTcpServer::IsOpen());

	for (FSession& Session : Sessions)
	{
		delete Session.Relay;
	}
}

////////////////////////////////////////////////////////////////////////////////
void FRecorder::Close()
{
	FAsioTickable::StopTick();
	FAsioTcpServer::Close();

	for (FSession& Session : Sessions)
	{
		Session.Relay->Close();
	}
}

////////////////////////////////////////////////////////////////////////////////
uint32 FRecorder::GetSessionCount() const
{
	return uint32(Sessions.Num());
}

////////////////////////////////////////////////////////////////////////////////
const FRecorder::FSession* FRecorder::GetSessionInfo(uint32 Index) const
{
	if (Index >= uint32(Sessions.Num()))
	{
		return nullptr;
	}

	return Sessions.GetData() + Index;
}

////////////////////////////////////////////////////////////////////////////////
bool FRecorder::OnAccept(asio::ip::tcp::socket& Socket)
{
	auto* Relay = new FRecorderRelay(Socket, Store);

	uint32 IdPieces[] = {
		Relay->GetIpAddress(),
		Socket.remote_endpoint().port(),
		Socket.local_endpoint().port(),
		0,
	};

	FSession Session;
	Session.Relay = Relay;
	Session.Id = QuickStoreHash(IdPieces);
	Sessions.Add(Session);

	return true;
}

////////////////////////////////////////////////////////////////////////////////
void FRecorder::OnTick()
{
	uint32 FinalNum = 0;
	for (int i = 0, n = Sessions.Num(); i < n; ++i)
	{
		FSession& Session = Sessions[i];
		if (Session.Relay->IsOpen())
		{
			Sessions[FinalNum] = Session;
			++FinalNum;
			continue;
		}

		delete Session.Relay;
	}

	Sessions.SetNum(FinalNum);
}

/* vim: set noexpandtab : */
