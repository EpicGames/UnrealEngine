// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Asio.h"
#include "AsioTcpServer.h"
#include "AsioTickable.h"
#include "Foundation.h"

class FStore;
class FRecorderRelay;

////////////////////////////////////////////////////////////////////////////////
class FRecorder
	: public FAsioTcpServer
	, public FAsioTickable
{
public:
	class FSession
	{
	public:
		uint32				GetId() const;
		uint32				GetTraceId() const;
		uint32				GetIpAddress() const;
		uint32				GetControlPort() const;
		const FGuid&		GetSessionGuid() const;
		const FGuid&		GetTraceGuid() const;

	private:
		friend				FRecorder;
		FRecorderRelay*		Relay;
		uint32				Id;
	};

							FRecorder(asio::io_context& IoContext, FStore& InStore);
							~FRecorder();
	void					Close();
	uint32					GetSessionCount() const;
	const FSession*			GetSessionInfo(uint32 Index) const;

private:
	virtual bool			OnAccept(asio::ip::tcp::socket& Socket) override;
	virtual void			OnTick() override;
	TArray<FSession>		Sessions;
	FStore&					Store;
};

/* vim: set noexpandtab : */
