// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "Store.h"
#include "StoreSettings.h"
#include <functional>

#if TS_USING(TS_PLATFORM_LINUX)
#   include <sys/inotify.h>
#	include <sys/types.h>
#	include <sys/stat.h>
#endif
#if TS_USING(TS_PLATFORM_MAC)
#	include <CoreServices/CoreServices.h>
#endif

#ifndef TS_DEBUG_FS_EVENTS
	#define TS_DEBUG_FS_EVENTS TS_OFF
#endif

////////////////////////////////////////////////////////////////////////////////
// Pre-C++20 there is now way to convert between clocks. C++20 onwards it is
// possible to create a clock with an Unreal epoch and convert file times to it.
// But at time of writing, C++20 isn't complete enough across the board.
static const uint64	UnrealEpochYear		= 1;
#if TS_USING(TS_PLATFORM_WINDOWS)
static const uint64	FsEpochYear			= 1601;
#else
static const uint64	FsEpochYear			= 1970;
#endif
static int64 FsToUnrealEpochBiasSeconds	= uint64(double(FsEpochYear - UnrealEpochYear) * 365.2425) * 86400;

////////////////////////////////////////////////////////////////////////////////
#if TS_USING(TS_PLATFORM_WINDOWS)
class FStore::FMount::FDirWatcher
	: public asio::windows::object_handle
{
public:
	using asio::windows::object_handle::object_handle;
};

////////////////////////////////////////////////////////////////////////////////
#elif TS_USING(TS_PLATFORM_LINUX)
class FStore::FMount::FDirWatcher
	: public asio::posix::stream_descriptor
{
	typedef std::function<void(asio::error_code error)> HandlerType;
public:
	using asio::posix::stream_descriptor::stream_descriptor;
	void async_wait(HandlerType InHandler)
	{
		asio::posix::stream_descriptor::async_wait(asio::posix::stream_descriptor::wait_read, InHandler);

		// Some new events have come into the stream. We need to read the events
		// event if we are not acting on the events themself (see Refresh for platform
		// independent update). If we don't read the data the next call to wait will
		// trigger immediately.
		bytes_readable AvailableCmd(true);
		io_control(AvailableCmd);
		size_t AvailableBytes = AvailableCmd.get();
		uint8* Buffer = (uint8*)malloc(AvailableBytes);
		read_some(asio::buffer(Buffer, AvailableBytes));
		
		#if TS_USING(TS_DEBUG_FS_EVENTS)
		size_t Cursor = 0;
		while(Cursor < AvailableBytes)
		{
			inotify_event *Event = (inotify_event *)Buffer + Cursor;
			printf("Recieved file event (0x%08x) ", Event->cookie);
			if (Event->len > 0 && strlen(Event->name))
				printf("on '%s': ", Event->name);
			else
				printf("on the directory ");
			if (Event->mask & IN_ACCESS)
				printf("ACCESS ");
			if ((Event->mask & IN_ATTRIB) != 0)
				printf("ATTRIB ");
			if ((Event->mask & IN_CLOSE_WRITE) != 0)
				printf("CLOSE_WRITE ");
			if ((Event->mask & IN_CLOSE_NOWRITE) != 0)
				printf("CLOSE_NOWRITE ");
			if ((Event->mask & IN_CREATE) != 0)
				printf("CREATE ");
			if ((Event->mask & IN_DELETE) != 0)
				printf("DELETE ");
			if ((Event->mask & IN_DELETE_SELF) != 0)
				printf("DELETE_SELF ");
			if ((Event->mask & IN_MODIFY) != 0)
				printf("MODIFY ");
			if ((Event->mask & IN_MOVE_SELF) != 0)
				printf("MOVE_SELF ");
			if ((Event->mask & IN_MOVED_FROM) != 0)
				printf("MOVED_FROM ");
			if ((Event->mask & IN_MOVED_TO) != 0)
				printf("MOVED_TO ");
			if ((Event->mask & IN_OPEN) != 0)
				printf("OPEN ");
			printf("\n");
			Cursor += sizeof(inotify_event) + Event->len;
		}
		#endif
	}
};

////////////////////////////////////////////////////////////////////////////////
#elif TS_USING(TS_PLATFORM_MAC)
class FStore::FMount::FDirWatcher
{
	typedef std::function<void(asio::error_code error)> HandlerType;
public:
	FDirWatcher(const char* InStoreDir)
	{
		std::error_code ErrorCode;
		StoreDir = std::filesystem::absolute(InStoreDir, ErrorCode);
		// Create watcher queue
		char WatcherName[128];
		snprintf(WatcherName, sizeof(WatcherName), "%s-%p", "FileWatcher", this);
		DispatchQueue = dispatch_queue_create(WatcherName, DISPATCH_QUEUE_SERIAL);
	}

	~FDirWatcher()
	{
		cancel();
	}

	void async_wait(HandlerType InHandler);
	void cancel()
	{
		close();
	}
	void close()
	{
		if (bIsRunning)
		{
			FSEventStreamStop(EventStream);
			FSEventStreamInvalidate(EventStream); // Also removed from dispatch queue
			FSEventStreamRelease(EventStream);
			dispatch_release(DispatchQueue);
			bIsRunning = false;
		}
	}
	bool is_open() { return bIsRunning; }

	void ProcessChanges(size_t EventCount, void* EventPaths, const FSEventStreamEventFlags EventFlags[])
	{
		bool bWatchedEvent = false;
		for (size_t EventIdx = 0; EventIdx < EventCount; ++EventIdx)
		{
			const FSEventStreamEventFlags& Flags = EventFlags[EventIdx];
#if TS_USING(TS_DEBUG_FS_EVENTS)
			const char** EventPathsArray = (const char**) EventPaths;
			const char* Path = EventPathsArray[EventIdx];
			printf("Recieved file event (%d) ", EventIdx);
			if (Path != nullptr)
				printf("on '%s': ", Path);
			else
				printf("on unknown file: ");
			if (Flags & kFSEventStreamEventFlagItemCreated)
				printf(" CREATED");
			if (Flags & kFSEventStreamEventFlagItemRemoved)
				printf(" REMOVED");
			if (Flags & kFSEventStreamEventFlagItemRenamed)
				printf(" RENAMED");
			printf("\n");
#endif
			constexpr unsigned int InterestingFlags = kFSEventStreamEventFlagItemCreated | 
				kFSEventStreamEventFlagItemRemoved | kFSEventStreamEventFlagItemRenamed;
			bWatchedEvent |= !!(Flags & InterestingFlags);
		}
		if (bWatchedEvent)
		{
			Handler(asio::error_code());
		}
	}

	FSEventStreamRef	EventStream;
	dispatch_queue_t	DispatchQueue;
	HandlerType 		Handler;
private:
	static void MacCallback(ConstFSEventStreamRef StreamRef,
					void* InDirWatcherPtr,
					size_t EventCount,
					void* EventPaths,
					const FSEventStreamEventFlags EventFlags[],
					const FSEventStreamEventId EventIDs[]);
	bool bIsRunning = false;
	FPath StoreDir;
};

////////////////////////////////////////////////////////////////////////////////
void FStore::FMount::FDirWatcher::MacCallback(ConstFSEventStreamRef StreamRef,
					void* InDirWatcherPtr,
					size_t EventCount,
					void* EventPaths,
					const FSEventStreamEventFlags EventFlags[],
					const FSEventStreamEventId EventIDs[])
{
	FStore::FMount::FDirWatcher* DirWatcherPtr = (FStore::FMount::FDirWatcher*)InDirWatcherPtr;
	check(DirWatcherPtr);
	check(DirWatcherPtr->EventStream == StreamRef);

	DirWatcherPtr->ProcessChanges(EventCount, EventPaths, EventFlags);
}

////////////////////////////////////////////////////////////////////////////////
void FStore::FMount::FDirWatcher::async_wait(HandlerType InHandler)
{
	if (bIsRunning)
	{
		return;
	}

	CFAbsoluteTime Latency = 0.2;	// seconds

	FSEventStreamContext Context;
	Context.version = 0;
	Context.info = this;
	Context.retain = NULL;
	Context.release = NULL;
	Context.copyDescription = NULL;

	// Set up streaming and turn it on
	CFStringRef FullPathMac = CFStringCreateWithFileSystemRepresentation(
		NULL,
		(const char*)StoreDir.c_str());
	CFArrayRef PathsToWatch = CFArrayCreate(NULL, (const void**)&FullPathMac, 1, NULL);

	// Create the event stream object
	EventStream = FSEventStreamCreate(
		NULL,
		&MacCallback,
		&Context,
		PathsToWatch,
		kFSEventStreamEventIdSinceNow,
		Latency,
		kFSEventStreamCreateFlagNoDefer|kFSEventStreamCreateFlagFileEvents
	);

	if (EventStream == nullptr)
	{
		printf("Failed to create file event stream for %s\n", CFStringGetCStringPtr(FullPathMac, kCFStringEncodingUnicode));
	}

	FSEventStreamSetDispatchQueue(EventStream, DispatchQueue);
	bIsRunning = FSEventStreamStart(EventStream);

	if (bIsRunning) 
	{
		printf("Watcher enabled on %s\n", StoreDir.c_str());
	}
	else
	{
		printf("Failed to start watcher for %s\n", StoreDir.c_str());
	}

	Handler = InHandler;
}
#endif



////////////////////////////////////////////////////////////////////////////////
FStore::FTrace::FTrace(const FPath& InPath)
: Path(InPath)
{
	const FString Name = GetName();
	Id = QuickStoreHash(InPath.c_str());

	// Calculate that trace's timestamp. Bias in seconds then convert to 0.1us.
	// Note: Asking for std::filesystem timestamp with epoch seems to work on Windows
	// and Mac, but behaves badly on Linux. Since in C++17 get file time epoch is
	// undefined in the standard we fall back to stat interface on Linux.
#if TS_USING(TS_PLATFORM_LINUX)
	struct stat FileStats;
	if (stat(Path.c_str(), &FileStats) || FileStats.st_mtime < 0)
	{
		// Failure code with errno
		Timestamp = 0;
		return;
	}
	Timestamp = FileStats.st_mtime;
#else
	std::error_code Ec;
	std::filesystem::file_time_type LastWriteTime = std::filesystem::last_write_time(Path, Ec);
	auto LastWriteDuration = LastWriteTime.time_since_epoch();
	Timestamp = std::chrono::duration_cast<std::chrono::seconds>(LastWriteDuration).count();
#endif
	Timestamp += FsToUnrealEpochBiasSeconds;
	Timestamp *= 10'000'000;

	// Check for .utrace.ini sidecar file
	FPath SidecarPath = Path;
	SidecarPath += ".ini"; // e.g., /path/MyTrace.utrace.ini
	std::error_code SidecarEc;
	if (std::filesystem::exists(SidecarPath, SidecarEc))
	{
		MetadataPath = SidecarPath;
	}
}

////////////////////////////////////////////////////////////////////////////////
FString FStore::FTrace::GetName() const
{
	return FString((const char*)Path.stem().u8string().c_str());
}

////////////////////////////////////////////////////////////////////////////////
const FPath& FStore::FTrace::GetPath() const
{
	return Path;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStore::FTrace::GetId() const
{
	return Id;
}

////////////////////////////////////////////////////////////////////////////////
uint64 FStore::FTrace::GetSize() const
{
	std::error_code Error;
	const uint64 Size = std::filesystem::file_size(Path, Error);
	return Error ? 0 : Size;
}

////////////////////////////////////////////////////////////////////////////////
uint64 FStore::FTrace::GetTimestamp() const
{
	return Timestamp;
}

////////////////////////////////////////////////////////////////////////////////
bool FStore::FTrace::HasMetadata() const
{
	return !MetadataPath.empty();
}

////////////////////////////////////////////////////////////////////////////////
FString FStore::FTrace::GetMetadataUri() const
{
	if (MetadataPath.empty())
	{
		return FString();
	}
	return fs::ToFString(MetadataPath);
}

////////////////////////////////////////////////////////////////////////////////
FStore::FMount* FStore::FMount::Create(FStore* InParent, asio::io_context& InIoContext, const FPath& InDir, bool bCreate)
{
	std::error_code ErrorCodeAbs, ErrorCodeCreate;
	// We would like to check if the path is absolute here, but Microsoft
	// has a very specific interpretation of what constitutes absolute path
	// which doesn't correspond to what UE's file utilities does. Hence paths
	// from Insights will not be correctly formatted.
	//const FPath AbsoluteDir = InDir.is_absolute() ? InDir : fs::absolute(InDir, ErrorCodeAbs);
	const FPath AbsoluteDir = InDir;
	if (bCreate)
	{
		// Make sure the directory exists
		fs::create_directories(AbsoluteDir, ErrorCodeCreate);
		if (ErrorCodeAbs || ErrorCodeCreate)
		{
			return nullptr;
		}
	}
	else
	{
		// If we are not allowed to create and if the directory does not exist
		// there is no point adding a mount for it.
		if (!fs::is_directory(AbsoluteDir))
		{
			return nullptr;
		}
	}
	return new FMount(InParent, InIoContext, AbsoluteDir);
}

////////////////////////////////////////////////////////////////////////////////
FStore::FMount::FMount(FStore* InParent, asio::io_context& InIoContext, const fs::path& InDir)
: Dir(InDir)
, Id(QuickStoreHash(InDir.c_str()))
, Parent(InParent)
#if TS_USING(TS_PLATFORM_WINDOWS) || TS_USING(TS_PLATFORM_LINUX)
, IoContext(InIoContext)
#endif
{

#if TS_USING(TS_PLATFORM_WINDOWS)
	std::wstring StoreDirW = Dir;
	HANDLE DirWatchHandle = FindFirstChangeNotificationW(StoreDirW.c_str(), false, FILE_NOTIFY_CHANGE_FILE_NAME|FILE_NOTIFY_CHANGE_DIR_NAME);
	if (DirWatchHandle == INVALID_HANDLE_VALUE)
	{
		DirWatchHandle = 0;
	}
	DirWatcher = new FDirWatcher(IoContext, DirWatchHandle);
#elif TS_USING(TS_PLATFORM_LINUX)
	int inotfd = inotify_init();
	int watch_desc = inotify_add_watch(inotfd, Dir.c_str(), IN_CREATE | IN_DELETE | IN_MOVED_FROM | IN_MOVED_TO);
	DirWatcher = new FDirWatcher(IoContext, inotfd);
#elif TS_USING(TS_PLATFORM_MAC)
	DirWatcher = new FDirWatcher(Dir.c_str());
#endif

	WatchDir();
}

////////////////////////////////////////////////////////////////////////////////
FStore::FMount::~FMount()
{
	ClearTraces();
	if (DirWatcher != nullptr)
	{
		delete DirWatcher;
	}
}

////////////////////////////////////////////////////////////////////////////////
FString FStore::FMount::GetDir() const
{
	return fs::ToFString(Dir);
}

////////////////////////////////////////////////////////////////////////////////
const FPath& FStore::FMount::GetPath() const
{
	return Dir;
}

////////////////////////////////////////////////////////////////////////////////
void FStore::FMount::Close()
{
	if (DirWatcher != nullptr)
	{
		DirWatcher->cancel();
		DirWatcher->close();
	}
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStore::FMount::GetId() const
{
	return Id;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStore::FMount::GetTraceCount() const
{
	return Traces.Num();
}

////////////////////////////////////////////////////////////////////////////////
const FStore::FTrace* FStore::FMount::GetTraceInfo(uint32 Index) const
{
	if (Index >= uint32(Traces.Num()))
	{
		return nullptr;
	}

	return Traces[Index];
}

////////////////////////////////////////////////////////////////////////////////
FStore::FTrace* FStore::FMount::GetTrace(uint32 Id) const
{
	for (FTrace* Trace : Traces)
	{
		if (Trace->GetId() == Id)
		{
			return Trace;
		}
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////
FStore::FTrace* FStore::FMount::AddTrace(const FPath& Path)
{
	FTrace NewTrace(Path);

	uint32 Id = NewTrace.GetId();
	if (FTrace* Existing = GetTrace(Id))
	{
		return Existing;
	}

	FTrace* Trace = new FTrace(MoveTemp(NewTrace));
	Traces.Add(Trace);
	return Trace;
}

////////////////////////////////////////////////////////////////////////////////
void FStore::FMount::ClearTraces()
{
	for (FTrace* Trace : Traces)
	{
		delete Trace;
	}
	Traces.Empty();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStore::FMount::Refresh()
{
	ClearTraces();

	uint32 ChangeSerial = 0;
	std::error_code Ec;
	for (auto& DirItem : std::filesystem::directory_iterator(Dir, Ec))
	{
		if (!DirItem.is_regular_file(Ec) && !DirItem.is_symlink(Ec))
		{
			continue;
		}

		FPath Extension = DirItem.path().extension();
		if (Extension != ".utrace")
		{
			continue;
		}

		FTrace* Trace = AddTrace(DirItem.path());

		if (Trace != nullptr)
		{
			ChangeSerial += Trace->GetId();
		}
	}
	return ChangeSerial;
}


void FStore::SetupMounts()
{
	AddMount(Settings->StoreDir, true);

	for (const FPath& Path : Settings->AdditionalWatchDirs)
	{
		AddMount(Path, false);
	}
}

////////////////////////////////////////////////////////////////////////////////
FStore::FStore(asio::io_context& InIoContext, const FStoreSettings* InSettings)
: IoContext(InIoContext)
, Settings(InSettings)
{
	SetupMounts();
	Refresh();
}

////////////////////////////////////////////////////////////////////////////////
FStore::~FStore()
{
	for (FMount* Mount : Mounts)
	{
		delete Mount;
	}
}

////////////////////////////////////////////////////////////////////////////////
void FStore::Close()
{
	for (FMount* Mount : Mounts)
	{
		Mount->Close();
	}
}

////////////////////////////////////////////////////////////////////////////////
bool FStore::AddMount(const FPath& Dir, bool bCreate)
{
	FMount* Mount = FMount::Create(this, IoContext, Dir, bCreate);
	if (Mount)
	{
		Mounts.Add(Mount);
	}
	return Mount != nullptr;
}

////////////////////////////////////////////////////////////////////////////////
bool FStore::RemoveMount(uint32 Id)
{
	for (uint32 i = 1, n = Mounts.Num() - 1; i <= n; ++i) // 1 because 0th must always exist
	{
		if (Mounts[i]->GetId() == Id)
		{
			FMount* MountToRemove = Mounts[i];
			std::swap(Mounts[n], Mounts[i]);
			Mounts.SetNum(n);
			delete MountToRemove;
			Refresh();
			return true;
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
void FStore::FMount::WatchDir()
{
	if (DirWatcher == nullptr)
	{
		return;
	}

	DirWatcher->async_wait([this] (asio::error_code ErrorCode)
	{
		if (ErrorCode)
		{
			return;
		}

#if TS_USING(TS_PLATFORM_WINDOWS)
		// Windows doesn't update modified timestamps in a timely fashion when
		// copying files (or it could be Explorer that doesn't update it until
		// later). This is a not-so-pretty "wait for a little bit" workaround.
		auto* DelayTimer = new asio::steady_timer(IoContext);
		DelayTimer->expires_after(std::chrono::seconds(2));
		DelayTimer->async_wait([this, DelayTimer] (const asio::error_code& ErrorCode)
		{
			delete DelayTimer;

			if (std::filesystem::exists(Dir))
			{
				Parent->Refresh();

				FindNextChangeNotification(DirWatcher->native_handle());
				WatchDir();
			}
		});
#else
	if (std::filesystem::exists(Dir))
	{
		Parent->Refresh();
		WatchDir();
	}
#endif
	});
}

////////////////////////////////////////////////////////////////////////////////
FString FStore::GetStoreDir() const
{
	FMount* Mount = Mounts[0];
	return Mount->GetDir();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStore::GetChangeSerial() const
{
	return ChangeSerial;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStore::GetTraceCount() const
{
	uint32 Count = 0;
	for (const FMount* Mount : Mounts)
	{
		Count += Mount->GetTraceCount();
	}
	return Count;
}

////////////////////////////////////////////////////////////////////////////////
const FStore::FTrace* FStore::GetTraceInfo(uint32 Index) const
{
	for (const FMount* Mount : Mounts)
	{
		uint32 Count = Mount->GetTraceCount();
		if (Index < Count)
		{
			return Mount->GetTraceInfo(Index);
		}
		Index -= Count;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////
FStore::FTrace* FStore::GetTrace(uint32 Id, FMount** OutMount) const
{
	for (FMount* Mount : Mounts)
	{
		if (FTrace* Trace = Mount->GetTrace(Id))
		{
			if (OutMount != nullptr)
			{
				*OutMount = Mount;
			}
			return Trace;
		}
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////
FStore::FNewTrace FStore::CreateTrace()
{
	// N.B. Not thread safe!?
	char Prefix[24];
	std::time_t Now = std::time(nullptr);
	std::tm* LocalNow = std::localtime(&Now);
	std::strftime(Prefix, TS_ARRAY_COUNT(Prefix), "%Y%m%d_%H%M%S", LocalNow);

	FMount* DefaultMount = Mounts[0];

	FPath TracePath(*DefaultMount->GetDir());
	TracePath /= Prefix;
	TracePath += ".utrace";
	
	for (uint32 Index = 0; std::filesystem::is_regular_file(TracePath); ++Index)
	{
		char FilenameIndexed[64];
		std::snprintf(FilenameIndexed, 64, "%s_%02d.utrace", Prefix, Index);
		TracePath.replace_filename(FPath(FilenameIndexed));
	}

	FAsioWriteable* File = FAsioFile::WriteFile(IoContext, TracePath);
	if (File == nullptr)
	{
		return {};
	}

	FTrace* Trace = DefaultMount->AddTrace(TracePath);
	if (Trace == nullptr)
	{
		delete File;
		return {};
	}

	return { Trace->GetId(), File };
}

////////////////////////////////////////////////////////////////////////////////
bool FStore::HasTrace(uint32 Id) const
{
	return GetTrace(Id) != nullptr;
}

////////////////////////////////////////////////////////////////////////////////
FAsioReadable* FStore::OpenTrace(uint32 Id)
{
	FMount* Mount;
	FTrace* Trace = GetTrace(Id, &Mount);
	if (Trace == nullptr)
	{
		return nullptr;
	}

	return FAsioFile::ReadFile(IoContext, Trace->GetPath());
}

////////////////////////////////////////////////////////////////////////////////
void FStore::OnSettingsChanged()
{
	// Remove all existing mounts
	for (const FMount* Mount : Mounts)
	{
		delete Mount;
	}
	Mounts.Empty();
	SetupMounts();
	Refresh();
}

////////////////////////////////////////////////////////////////////////////////
void FStore::Refresh()
{
	ChangeSerial = 0;
	for (FMount* Mount : Mounts)
	{
		ChangeSerial += Mount->Refresh();
	}
}

/* vim: set noexpandtab : */
