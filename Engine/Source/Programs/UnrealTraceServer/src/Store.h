// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Asio.h"
#include "AsioFile.h"
#include "Foundation.h"
#include "Utils.h"

class FAsioReadable;
class FAsioWriteable;
class FDirWatcher;
class FStoreSettings;

////////////////////////////////////////////////////////////////////////////////
enum class EStoreVersion
{
	Value = 0x0100, // 0xMMmm MM=major, mm=minor
};

////////////////////////////////////////////////////////////////////////////////
class FStore
{
public:
	class FTrace
	{
	public:
							FTrace(const FPath& InPath);
		const FPath&		GetPath() const;
		FString				GetName() const;
		uint32				GetId() const;
		uint64				GetSize() const;
		uint64				GetTimestamp() const;
		bool				HasMetadata() const;
		FString				GetMetadataUri() const;

	private:
		friend				FStore;
		FPath				Path;
		uint64				Timestamp;
		uint32				Id = 0;
		FPath				MetadataPath; // Path to .utrace.ini sidecar (empty if not found)
	};

	struct FNewTrace
	{
		uint32			Id;
		FAsioWriteable* Writeable;
	};

	class FMount
	{
	public:
		static FMount* Create(FStore* InParent, asio::io_context& InIoContext, const FPath& InDir, bool bCreate);
						
		void			Close();
		uint32			GetId() const;
		FString			GetDir() const;
		const FPath&	GetPath() const;
		uint32			GetTraceCount() const;
		const FTrace*	GetTraceInfo(uint32 Index) const;

	private:
		class			FDirWatcher;
		friend			FStore;
						FMount(FStore* InParent, asio::io_context& InIoContext, const FPath& InDir);
						~FMount();
		FTrace*			GetTrace(uint32 Id) const;
		FTrace*			AddTrace(const FPath& Path);
		void			ClearTraces();
		uint32			Refresh();
		void			WatchDir();
		
		FPath			Dir;
		TArray<FTrace*>	Traces;
		uint32			Id;
		FDirWatcher*	DirWatcher = nullptr;
		FStore*			Parent = nullptr;
#if TS_USING(TS_PLATFORM_WINDOWS) || TS_USING(TS_PLATFORM_LINUX)
		asio::io_context& IoContext;
#endif
	};

	void SetupMounts();
						FStore(asio::io_context& IoContext, const FStoreSettings* InSettings);
						~FStore();
	void				Close();
	FString				GetStoreDir() const;
	uint32				GetChangeSerial() const;
	uint32				GetTraceCount() const;
	const FTrace*		GetTraceInfo(uint32 Index) const;
	bool				HasTrace(uint32 Id) const;
	FNewTrace			CreateTrace();
	FAsioReadable*		OpenTrace(uint32 Id);
	void				OnSettingsChanged();

private:
	bool				AddMount(const FPath& Dir, bool bCreate);
	bool				RemoveMount(uint32 Id);
	FTrace*				GetTrace(uint32 Id, FMount** OutMount=nullptr) const;
	void				Refresh();
	
	asio::io_context&	IoContext;
	TArray<FMount*>		Mounts;
	uint32				ChangeSerial;
	const FStoreSettings* Settings;
};

/* vim: set noexpandtab : */
