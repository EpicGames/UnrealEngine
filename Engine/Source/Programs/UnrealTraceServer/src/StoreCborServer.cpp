// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "AsioIoable.h"
#include "AsioSocket.h"
#include "CborPayload.h"
#include "Recorder.h"
#include "Store.h"
#include "StoreCborServer.h"
#include "StoreService.h"
#include "StoreSettings.h"
#include "TraceRelay.h"
#include "Version.h"

////////////////////////////////////////////////////////////////////////////////
class FStoreCborPeer
	: public FAsioIoSink
{
public:
							FStoreCborPeer(asio::ip::tcp::socket& InSocket, FStore& InStore, FRecorder& InRecorder, FStoreCborServer& InParent);
	virtual					~FStoreCborPeer();
	bool					IsOpen() const;
	void					Close();

protected:
	void					OnPayload();
	void					OnSessionCount();
	void					OnSessionInfo();
	void					OnStatus();
	void					OnVersion();
	void					OnTraceCount();
	void					OnTraceInfo();
	void					OnTraceRead();
	void					OnSettingsWrite();
	void					SendError(EStatusCode StatusCode);
	void					SendResponse(const FPayload& Payload);
	virtual void			OnIoComplete(uint32 Id, int32 Size) override;
	enum					{ OpStart, OpReadPayloadSize, OpReadPayload, OpSendResponse };
	FStore&					Store;
	FRecorder&				Recorder;
	FStoreCborServer&		Parent;
	FAsioSocket				Socket;
	uint32					PayloadSize;
	FResponse				Response;
};

////////////////////////////////////////////////////////////////////////////////
FStoreCborPeer::FStoreCborPeer(
	asio::ip::tcp::socket& InSocket,
	FStore& InStore,
	FRecorder& InRecorder,
	FStoreCborServer& InParent)
: Store(InStore)
, Recorder(InRecorder)
, Parent(InParent)
, Socket(InSocket)
{
	OnIoComplete(OpStart, 0);
}

////////////////////////////////////////////////////////////////////////////////
FStoreCborPeer::~FStoreCborPeer()
{
	check(!IsOpen());
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborPeer::IsOpen() const
{
	return Socket.IsOpen();
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborPeer::Close()
{
	Socket.Close();
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborPeer::OnSessionCount()
{
	TPayloadBuilder<> Builder(EStatusCode::Success);
	Builder.AddInteger("count", Recorder.GetSessionCount());
	SendResponse(Builder.Done());
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborPeer::OnSessionInfo()
{
	const FRecorder::FSession* Session = nullptr;

	int32 Index = int32(Response.GetInteger("index", -1));
	if (Index >= 0)
	{
		Session = Recorder.GetSessionInfo(Index);
	}
	else if (uint32 Id = uint32(Response.GetInteger("id", 0)))
	{
		for (int i = 0, n = Recorder.GetSessionCount(); i < n; ++i)
		{
			const FRecorder::FSession* Candidate = Recorder.GetSessionInfo(i);
			if (Candidate->GetId() == Id)
			{
				Session = Candidate;
				break;
			}
		}
	}
	else if (uint32 TraceId = uint32(Response.GetInteger("trace_id", 0)))
	{
		for (int i = 0, n = Recorder.GetSessionCount(); i < n; ++i)
		{
			const FRecorder::FSession* Candidate = Recorder.GetSessionInfo(i);
			if (Candidate->GetTraceId() == TraceId)
			{
				Session = Candidate;
				break;
			}
		}
	}
	else if (FStringView TraceGuidStr = Response.GetString("trace_guid", ""); TraceGuidStr.Len() > 0)
	{
		FGuid TraceGuid;
		if (FGuid::ParseGuid(TraceGuidStr, TraceGuid))
		{
			for (int i = 0, n = Recorder.GetSessionCount(); i < n; ++i)
			{
				const FRecorder::FSession* Candidate = Recorder.GetSessionInfo(i);

					if (FGuid::Equal(Candidate->GetTraceGuid(), TraceGuid))
					{
						Session = Candidate;
						break;
					}
			}
		}
	}

	if (Session == nullptr)
	{
		return SendError(EStatusCode::BadRequest);
	}

	TPayloadBuilder<> Builder(EStatusCode::Success);
	Builder.AddInteger("id", Session->GetId());
	Builder.AddInteger("trace_id", Session->GetTraceId());
	Builder.AddInteger("ip_address", Session->GetIpAddress());
	Builder.AddInteger("control_port", Session->GetControlPort());
	SendResponse(Builder.Done());
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborPeer::OnStatus()
{
	const FStoreSettings* Settings = Parent.GetSettings();
	TPayloadBuilder<> Builder(EStatusCode::Success);
	Builder.AddInteger("recorder_port", Settings->RecorderPort);
	Builder.AddInteger("store_port", Settings->StorePort);
	Builder.AddInteger("sponsored", Settings->Sponsored);
	Builder.AddInteger("change_serial", Store.GetChangeSerial());
	Builder.AddInteger("settings_serial", Settings->GetChangeSerial());
	Builder.AddString("store_dir", Settings->StoreDir.string().c_str());
	TArray<FString> AdditionalWatchDirs;
	for (const fs::path& Path : Settings->AdditionalWatchDirs)
	{
		AdditionalWatchDirs.Add(Path.string());
	}
	Builder.AddStringArray("watch_dirs", AdditionalWatchDirs);
	SendResponse(Builder.Done());
}

///////////////////////////////////////////////////////////////////////////////
void FStoreCborPeer::OnVersion()
{
	TPayloadBuilder<> Builder(EStatusCode::Success);
	Builder.AddInteger("major", TS_VERSION_PROTOCOL);
	Builder.AddInteger("minor", TS_VERSION_MINOR);
#if TS_USING(TS_BUILD_DEBUG)
	const char* Configuration = "Debug";
#else
	const char* Configuration = "Release";
#endif
	Builder.AddString("configuration", Configuration);
	SendResponse(Builder.Done());
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborPeer::OnTraceCount()
{
	TPayloadBuilder<> Builder(EStatusCode::Success);
	Builder.AddInteger("count", Store.GetTraceCount());
	SendResponse(Builder.Done());
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborPeer::OnTraceInfo()
{
	const FStore::FTrace* Trace = nullptr;

	int32 Index = int32(Response.GetInteger("index", -1));
	if (Index >= 0)
	{
		Trace = Store.GetTraceInfo(Index);
	}
	else
	{
		uint32 Id = uint32(Response.GetInteger("id", 0));
		if (Id != 0)
		{
			for (int i = 0, n = Store.GetTraceCount(); i < n; ++i)
			{
				const FStore::FTrace* Candidate = Store.GetTraceInfo(i);
				if (Candidate->GetId() == Id)
				{
					Trace = Candidate;
					break;
				}
			}
		}
	}

	if (Trace == nullptr)
	{
		return SendError(EStatusCode::BadRequest);
	}

	auto Name = Trace->GetName();
	auto PathString = Trace->GetPath().string();

	TPayloadBuilder<> Builder(EStatusCode::Success);
	Builder.AddInteger("id", Trace->GetId());
	Builder.AddInteger("size", Trace->GetSize());
	Builder.AddInteger("timestamp", Trace->GetTimestamp());
	Builder.AddString("name", *Name);
	Builder.AddString("uri", PathString.c_str());
	if (Trace->HasMetadata())
	{
		auto MetadataUri = Trace->GetMetadataUri();
		Builder.AddString("metadata_uri", *MetadataUri);
	}
	SendResponse(Builder.Done());
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborPeer::OnTraceRead()
{
	uint32 Id = uint32(Response.GetInteger("id", 0));
	if (!Id || !Store.HasTrace(Id))
	{
		return SendError(EStatusCode::BadRequest);
	}

	FTraceRelay* Relay = Parent.RelayTrace(Id);
	if (Relay == nullptr)
	{
		return SendError(EStatusCode::InternalError);
	}

	TPayloadBuilder<> Builder(EStatusCode::Success);
	Builder.AddInteger("port", Relay->GetPort());
	SendResponse(Builder.Done());
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborPeer::OnSettingsWrite()
{
	// Only allow local connections to change settings
	if (!Socket.IsLocalConnection())
	{
		return SendError(EStatusCode::Forbidden);
	}
	// Copy the current settings and apply requested changes. This is possible
	// to do without synchronization since asio currently only runs on a single
	// thread, so no other operations should be running at the same time.
	FStoreSettings* Settings = Parent.GetSettings();
	Settings->ApplySettingsFromCbor(Response.GetData(), Response.GetSize());
	// Save settings to disk
	Settings->WriteToSettingsFile();

	TPayloadBuilder<> Builder(EStatusCode::Success);
	SendResponse(Builder.Done());

	// Let all the service components know settings have changed
	Parent.OnSettingsChanged();
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborPeer::OnPayload()
{
	FStringView Request = Response.GetString("$request", "");
	FStringView Path = Response.GetString("$path", "");
	if (!Request.Len() || !Path.Len())
	{
		SendError(EStatusCode::BadRequest);
		return;
	}

	static struct {
		uint32	Hash;
		void	(FStoreCborPeer::*Func)();
	} const DispatchTable[] = {
		{ QuickStoreHash("v1/session/count"),	&FStoreCborPeer::OnSessionCount },
		{ QuickStoreHash("v1/session/info"),	&FStoreCborPeer::OnSessionInfo },
		{ QuickStoreHash("v1/status"),			&FStoreCborPeer::OnStatus },
		{ QuickStoreHash("v1/version"),			&FStoreCborPeer::OnVersion },
		{ QuickStoreHash("v1/trace/count"),		&FStoreCborPeer::OnTraceCount },
		{ QuickStoreHash("v1/trace/info"),		&FStoreCborPeer::OnTraceInfo },
		{ QuickStoreHash("v1/trace/read"),		&FStoreCborPeer::OnTraceRead },
		{ QuickStoreHash("v1/settings/write"),	&FStoreCborPeer::OnSettingsWrite },
	};

	uint32 PathHash = QuickStoreHash(Path);
	for (const auto& Row : DispatchTable)
	{
		if (Row.Hash == PathHash)
		{
			return (this->*(Row.Func))();
		}
	}

	SendError(EStatusCode::NotFound);
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborPeer::SendError(EStatusCode StatusCode)
{
	TPayloadBuilder<> Builder(StatusCode);
	FPayload Payload = Builder.Done();
	SendResponse(Payload);
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborPeer::SendResponse(const FPayload& Payload)
{
	struct {
		uint32	Size;
		uint8	Data[];
	}* Dest;

	Dest = decltype(Dest)(Response.Reserve(sizeof(*Dest) + Payload.Size));
	Dest->Size = Payload.Size;
	memcpy(Dest->Data, Payload.Data, Payload.Size);
	Socket.Write(Dest, sizeof(*Dest) + Payload.Size, this, OpSendResponse);
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborPeer::OnIoComplete(uint32 Id, int32 Size)
{
	if (Size < 0)
	{
		Close();
		return;
	}

	switch (Id)
	{
	case OpStart:
	case OpSendResponse:
		Socket.Read(&PayloadSize, sizeof(uint32), this, OpReadPayloadSize);
		break;

	case OpReadPayloadSize:
		if (PayloadSize == 0 || PayloadSize > 1024)
		{
			SendError(EStatusCode::BadRequest);
			break;
		}
		Socket.Read(Response.Reserve(PayloadSize), PayloadSize, this, OpReadPayload);
		break;

	case OpReadPayload:
		OnPayload();
		break;
	}
}



////////////////////////////////////////////////////////////////////////////////
FStoreCborServer::FStoreCborServer(
	asio::io_context& IoContext,
	FStoreSettings* InSettings,
	FStore& InStore,
	FRecorder& InRecorder)
: FAsioTcpServer(IoContext)
, FAsioTickable(IoContext)
, Store(InStore)
, Recorder(InRecorder)
, Settings(InSettings)
{
	if (!StartServer(Settings->StorePort))
	{
		StartServer();
	}
	StartTick(500);
}

////////////////////////////////////////////////////////////////////////////////
FStoreCborServer::~FStoreCborServer()
{
	check(!FAsioTcpServer::IsOpen());
	check(!FAsioTickable::IsActive());

	for (FStoreCborPeer* Peer : Peers)
	{
		delete Peer;
	}

	for (FTraceRelay* Relay : Relays)
	{
		delete Relay;
	}
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborServer::Close()
{
	FAsioTcpServer::Close();
	FAsioTickable::StopTick();

	for (FStoreCborPeer* Peer : Peers)
	{
		Peer->Close();
	}

	for (FTraceRelay* Relay : Relays)
	{
		Relay->Close();
	}
}

////////////////////////////////////////////////////////////////////////////////
FStore& FStoreCborServer::GetStore() const
{
	return Store;
}

////////////////////////////////////////////////////////////////////////////////
FRecorder& FStoreCborServer::GetRecorder() const
{
	return Recorder;
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborServer::OnAccept(asio::ip::tcp::socket& Socket)
{
	FStoreCborPeer* Peer = new FStoreCborPeer(Socket, Store, Recorder, *this);
	Peers.Add(Peer);
	return true;
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborServer::OnTick()
{
	// Clean up dead peers
	uint32 FinalNum = 0;
	for (int i = 0, n = Peers.Num(); i < n; ++i)
	{
		FStoreCborPeer* Peer = Peers[i];
		if (Peer->IsOpen())
		{
			Peers[FinalNum] = Peer;
			++FinalNum;
			continue;
		}

		delete Peer;
	}
	Peers.SetNum(FinalNum);

	// Clean up dead relays
	FinalNum = 0;
	for (int i = 0, n = Relays.Num(); i < n; ++i)
	{
		FTraceRelay* Relay = Relays[i];
		if (Relay->IsOpen())
		{
			Relays[FinalNum] = Relay;
			++FinalNum;
			continue;
		}

		delete Relay;
	}
	Relays.SetNum(FinalNum);
}

////////////////////////////////////////////////////////////////////////////////
FTraceRelay* FStoreCborServer::RelayTrace(uint32 Id)
{
	FAsioReadable* Input = Store.OpenTrace(Id);
	if (Input == nullptr)
	{
		return nullptr;
	}

	uint32 SessionId = 0;
	for (int i = 0, n = Recorder.GetSessionCount(); i < n; ++i)
	{
		const FRecorder::FSession* Session = Recorder.GetSessionInfo(i);
		if (Session->GetTraceId() == Id)
		{
			SessionId = Session->GetId();
			break;
		}
	}

	asio::io_context& IoContext = FAsioTcpServer::GetIoContext();
	FTraceRelay* Relay = new FTraceRelay(IoContext, Input, SessionId, Recorder);
	if (Relay->GetPort() == 0)
	{
		delete Relay;
		return nullptr;
	}

	Relays.Add(Relay);
	return Relay;
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborServer::OnSettingsChanged()
{
	Store.OnSettingsChanged();
}

/* vim: set noexpandtab : */
