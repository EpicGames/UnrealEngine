// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Asio.h"
#include "AsioTcpServer.h"
#include "AsioTickable.h"
#include "Foundation.h"

class FRecorder;
class FStore;
class FStoreCborPeer;
class FTraceRelay;
class FStoreSettings;

////////////////////////////////////////////////////////////////////////////////
class FStoreCborServer
	: public FAsioTcpServer
	, public FAsioTickable
{
	friend class FStoreCborPeer;

public:
							FStoreCborServer(asio::io_context& IoContext, FStoreSettings* InSettings, FStore& InStore, FRecorder& InRecorder);
							~FStoreCborServer();
	void					Close();
	FStore&					GetStore() const;
	FRecorder&				GetRecorder() const;
	FTraceRelay*			RelayTrace(uint32 Id);
	void					OnSettingsChanged();
	uint32					GetActivePeerCount() const { return Peers.Num(); }

private:
	virtual bool			OnAccept(asio::ip::tcp::socket& Socket) override;
	virtual void			OnTick() override;
	FStoreSettings*			GetSettings() const { return Settings; }

	TArray<FStoreCborPeer*>	Peers;
	TArray<FTraceRelay*>	Relays;
	FStore&					Store;
	FRecorder&				Recorder;
	FStoreSettings*			Settings;
};

/* vim: set noexpandtab : */
