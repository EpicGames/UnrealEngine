// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "Asio.h"
#include "AsioContext.h"
#include "InstanceInfo.h"
#include "Lifetime.h"
#include "Recorder.h"
#include "Store.h"
#include "StoreCborServer.h"
#include "StoreService.h"
#include "StoreSettings.h"

////////////////////////////////////////////////////////////////////////////////
struct FStoreServiceImpl
{
public:
							FStoreServiceImpl(FStoreSettings* Settings, FInstanceInfo* InstanceInfo);
							~FStoreServiceImpl();
	FAsioContext			Context;
	FStore					Store;
	FRecorder				Recorder;
	FStoreCborServer		CborServer;
	FLifetime				LifetimeManager;
};

////////////////////////////////////////////////////////////////////////////////
FStoreServiceImpl::FStoreServiceImpl(FStoreSettings* Settings, FInstanceInfo* InstanceInfo)
: Context(Settings->ThreadCount)
, Store(Context.Get(), Settings)
, Recorder(Context.Get(), Store)
, CborServer(Context.Get(), Settings, Store, Recorder)
, LifetimeManager(Context.Get(), (FStoreService*)this, Settings, InstanceInfo)
{
	if (Settings->RecorderPort >= 0)
	{
		Recorder.StartServer(Settings->RecorderPort);
	}
	Context.Start();
}

////////////////////////////////////////////////////////////////////////////////
FStoreServiceImpl::~FStoreServiceImpl()
{
	asio::post(Context.Get(), [this] () {
		LifetimeManager.StopTick();
		CborServer.Close();
		Recorder.Close();
		Store.Close();
	});
	Context.Wait();
}



////////////////////////////////////////////////////////////////////////////////
FStoreService* FStoreService::Create(FStoreSettings* Settings, FInstanceInfo* InstanceInfo)
{
	if (Settings->ThreadCount <= 0)
	{
		Settings->ThreadCount = std::thread::hardware_concurrency();
	}

	// TODO: not thread safe yet
	Settings->ThreadCount = 1;

	return (FStoreService*) new FStoreServiceImpl(Settings, InstanceInfo);
}

////////////////////////////////////////////////////////////////////////////////
void FStoreService::operator delete (void* Addr)
{
	auto* Self = (FStoreServiceImpl*)Addr;
	delete Self;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreService::GetPort() const
{
	auto* Self = (FStoreServiceImpl*)this;
	return Self->CborServer.GetPort();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreService::GetRecorderPort() const
{
	auto* Self = (FStoreServiceImpl*)this;
	return Self->Recorder.GetPort();
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreService::ShutdownIfNoConnections()
{
	auto* Self = (FStoreServiceImpl*)this;
	const uint32 ConnectionCount = Self->Recorder.GetSessionCount() + Self->CborServer.GetActivePeerCount();
	if (!ConnectionCount)
	{
		// Close the recorder and store server so new connections are not
		// accepted between returning and process exiting.
		Self->Recorder.Close();
		Self->CborServer.Close();
		return true;
	}
	return false;
}

/* vim: set noexpandtab : */
