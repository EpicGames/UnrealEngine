// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Foundation.h"

class FStoreSettings;

////////////////////////////////////////////////////////////////////////////////
class FStoreService
{
public:
							~FStoreService() = default;
	static FStoreService*	Create(FStoreSettings* Desc, struct FInstanceInfo* InstanceInfo);
	void					operator delete (void* Addr);
	uint32					GetPort() const;
	uint32					GetRecorderPort() const;
	bool					ShutdownIfNoConnections();

private:
							FStoreService() = default;
							FStoreService(const FStoreService&) = delete;
							FStoreService(const FStoreService&&) = delete;
	void					operator = (const FStoreService&) = delete;
	void					operator = (const FStoreService&&) = delete;
};

/* vim: set noexpandtab : */
