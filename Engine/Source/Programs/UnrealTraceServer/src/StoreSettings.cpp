// Copyright Epic Games, Inc. All Rights Reserved.
#include "Pch.h"
#include "Cbor.h"
#include "Logging.h"
#include "StoreSettings.h"
#include "Utils.h"
#include <charconv>

////////////////////////////////////////////////////////////////////////////////
static const char* GSettingsFilename = "Settings.ini";
static const char* GStoreDirName = "StoreDir";
static const char* GAdditionalWatchDirsName = "Additionalwatchdirs";
static const char* GStorePortName = "StorePort";
static const char* GRecorderPortName = "RecorderPort";
static const char* GThreadCountName = "ThreadCount";
static const char* GSponsoredName = "Sponsored";

////////////////////////////////////////////////////////////////////////////////

// Logging macro
#if 1
#define TS_SETTINGS_LOGF(Fmt, ...) TS_LOG(Fmt, __VA_ARGS__);
#define TS_SETTINGS_LOG(String) TS_LOG(String);
#else
#define TS_SETTINGS_LOGF(...)
#define TS_SETTINGS_LOG(...)
#endif

// Useful macro for tracing changes to settings
#if 0
#define TS_SETTINGS_TRACE(Fmt, ...) TS_SETTINGS_LOGF(Fmt, __VA_ARGS__)
#else
#define TS_SETTINGS_TRACE(Fmt, ...)
#endif

// Formatted text printing used in FIniWriter
#define PRINTCHECKEDF(Handle, Format, ...) {\
	const int Result = fprintf(Handle, Format, __VA_ARGS__);\
	if (Result <= 0) { return false; }\
	}\
	
#define PRINTCHECKED(Handle, String) {\
	const int Result = fprintf(Handle, String);\
	if (Result <= 0) { return false; }\
	}\

////////////////////////////////////////////////////////////////////////////////
class FIniWriter
{
public:
	FIniWriter(const FPath& Path)
	{
		Handle = std::fopen((const char*) Path.string().c_str(), "w");
	}

	~FIniWriter()
	{
		if (Handle)
		{
			std::fclose(Handle);
		}
	}
	
	bool WritePath(const char* Ident, const FPath& Value) const
	{
		if (Ident != nullptr && !Value.empty())
		{
			PRINTCHECKEDF(Handle, "%s=%s\n", Ident, Value.string().c_str())
			return true;
		}
		return false;
	}

	bool WriteInteger(const char* Ident, int64 Value) const
	{
		if (Ident != nullptr)
		{
			PRINTCHECKEDF(Handle, "%s=%lld\n", Ident, (long long int) Value)
			return true;
		}
		return false;
	}

	bool WritePathArray(const char* Ident, const TArray<FPath>& Value) const
	{
		if (Ident != nullptr && Value.Num() > 0)
		{
			bool bFirst = true;
			PRINTCHECKEDF(Handle, "%s=", Ident)
			for (const FPath& ArrayValue : Value)
			{
				PRINTCHECKEDF(Handle, "%s %s", bFirst ? "" : ",", ArrayValue.string().c_str())
				bFirst = false;
			}
			PRINTCHECKED(Handle, "\n")
			return true;
		}
		return Value.Num() == 0;
	}
	

private:
	FILE* Handle;
};

////////////////////////////////////////////////////////////////////////////////
class FIniReader
{
public:	
	FIniReader(const FPath& Path)
	{
		FILE* Handle = std::fopen((const char*) Path.string().c_str(), "r");
		if (!Handle)
		{
			TS_SETTINGS_LOGF("Unable to open settings file %s.\n", Path.string().c_str())
			return; // Safe to return here, buffer and entries will be empty
		}

		// Read entire file into a buffer
		std::fseek(Handle, 0, SEEK_END);
		const size_t FileSize = std::ftell(Handle);
		Buffer.SetNum(int32(FileSize));
		std::fseek(Handle, 0, 0);
		const size_t Read = std::fread(Buffer.GetData(), sizeof(uint8), FileSize, Handle);
		if (!Read)
		{
			TS_SETTINGS_LOGF("Unable to read settings file %s.\n", Path.string().c_str())
			std::fclose(Handle);
			return;
		}

		// Parse all the lines and find identifier=value expressions
		std::string_view BufferView((const char*) Buffer.GetData(), Buffer.Num());
		std::string_view Line = BufferView.substr(0, BufferView.find_first_of('\n'));
		while (!Line.empty())
		{
			const size_t EqualPos = Line.find_first_of('=');
			if (EqualPos != std::string_view::npos && Line.size() > (EqualPos + 1))
			{
				const std::string_view Ident = Line.substr(0, EqualPos);
				const std::string_view Value = Line.substr(EqualPos + 1);
				Entries.Add(Entry {Ident, Value});
			}
				
			BufferView.remove_prefix(std::min(BufferView.size(), Line.size() + 1));
			Line = BufferView.substr(0, BufferView.find_first_of('\n'));
		}

		std::fclose(Handle);
	}

	void GetPath(const char* Ident, FPath& OutPath)
	{
		GetIdentifier(Ident, [&](std::string_view Value)
		{
			TS_SETTINGS_TRACE("Set -> %s to %.*s\n", Ident, int(Value.length()), Value.data())
			OutPath = TrimWhitespace(Value);
		});
	}

	void GetPathArray(const char* Ident, TArray<FPath>& OutArray)
	{
		GetIdentifier(Ident, [&](std::string_view Value)
		{
			std::string_view View(Value);
			std::string_view Path = View.substr(0, View.find_first_of(','));
			while (!Path.empty())
			{
				TS_SETTINGS_TRACE("Add -> %s '%.*s'\n", Ident, int(Value.length()), Value.data())
				OutArray.Add(FPath(TrimWhitespace(Path)));
				// remove_suffix has no range check
				View.remove_prefix(std::min(View.size(), Path.size()+1));
				Path = View.substr(0, View.find_first_of(','));
			}
		});
	}

	void GetInteger(const char* Ident, int32& OutValue)
	{
		GetIdentifier(Ident, [&](std::string_view Value)
		{
			int32 Parsed;
			auto [ptr, ec] = std::from_chars(Value.data(), Value.data() + Value.size(), Parsed);
			if (ec == std::errc())
			{
				TS_SETTINGS_TRACE("Set -> %s to %d\n", Ident, Parsed)
				OutValue = Parsed;
			}
		});
	}

private:

	struct Entry
	{
		std::string_view Ident;
		std::string_view Value;
	};
	
	template<typename LambdaType>
	void GetIdentifier(const char* InIdent, LambdaType Callback)
	{
		const std::string_view InIdentView(InIdent);
		auto EntryIt = std::find_if(Entries.begin(), Entries.end(), [&](const Entry& Entry){ 
			return EqualCaseInsensitive(Entry.Ident, InIdentView); 
		});
		if (EntryIt != Entries.end())
		{
			const Entry& Entry = *EntryIt;
			Callback(Entry.Value);
		}
	}

	static std::string_view TrimWhitespace(const std::string_view& Input)
	{
		std::string_view Output(Input);
		const size_t Count = Output.length() - 1;
		size_t PreTrim(0), PostTrim(Count);
		while (PreTrim < PostTrim && std::isspace(Output.at(PreTrim)))
		{
			++PreTrim;
		}
		while (PostTrim > PreTrim && std::isspace(Output.at(PostTrim)))
		{
			--PostTrim;
		}
		if (PostTrim < Count)
		{
			Output.remove_suffix(Count - PostTrim);
		}
		if (PreTrim > 0)
		{
			Output.remove_prefix(PreTrim);
		}
		
		return Output;
	}

	static bool EqualCaseInsensitive(const std::string_view& Lhs, const std::string_view& Rhs)
	{
		if (Lhs.size() != Rhs.size())
		{
			return false;
		}
		for (uint32 i = 0; i < Lhs.size(); ++i)
		{
			if (std::tolower(Lhs.at(i)) != std::tolower(Rhs.at(i)))
			{
				return false;
			}
		}
		return true;
	}

	TArray<uint8> Buffer;
	TArray<Entry> Entries;
};

////////////////////////////////////////////////////////////////////////////////
void FStoreSettings::ReadFromSettings(const FPath& Path)
{
	// Create expected filepath and normalize
	const FPath FilePath = fs::absolute(Path / fs::path(GSettingsFilename));

	// Save the file path to the settings file so we can update it down the line
	SettingsFile = FilePath.string();

	// Set default store directory. Will be overwritten by settings
	// Use the legacy '001' folder here to avoid changing existing stores.
	StoreDir = fs::absolute(Path / fs::path("Store/001"));
	
	TS_SETTINGS_LOGF("Reading settings from '%s'\n", SettingsFile.string().c_str())

	FIniReader Reader(FilePath);
	Reader.GetPath(GStoreDirName, StoreDir);
	Reader.GetPathArray(GAdditionalWatchDirsName, AdditionalWatchDirs);
	Reader.GetInteger(GStorePortName, StorePort);
	Reader.GetInteger(GRecorderPortName, RecorderPort);
	Reader.GetInteger(GThreadCountName, ThreadCount);
	Reader.GetInteger(GSponsoredName, Sponsored);

	++ChangeSerial;
}

////////////////////////////////////////////////////////////////////////////////
void FStoreSettings::WriteToSettingsFile() const
{
	if (SettingsFile.empty())
	{
		TS_SETTINGS_LOG("No settings file set.'\n")
		return;
	}

	TS_SETTINGS_LOGF("Writing settings to '%s'\n", SettingsFile.string().c_str())

	const FIniWriter Writer(SettingsFile);
	bool bOk = true;
	bOk &= Writer.WritePath(GStoreDirName, StoreDir.string());
	bOk &= Writer.WritePathArray(GAdditionalWatchDirsName, AdditionalWatchDirs);
	bOk &= Writer.WriteInteger(GStorePortName, StorePort);
	bOk &= Writer.WriteInteger(GRecorderPortName, RecorderPort);
	bOk &= Writer.WriteInteger(GThreadCountName, ThreadCount);
	bOk &= Writer.WriteInteger(GSponsoredName, Sponsored);

	if (!bOk)
	{
		TS_SETTINGS_LOG("Errors occurred while write settings file.\n")
	}
}

////////////////////////////////////////////////////////////////////////////////
void FStoreSettings::ApplySettingsFromCbor(const uint8* Buffer, uint32 NumBytes)
{
	#define IS_PROPERTY(name, type) Prop.Compare(name) == 0 && Reader.ReadNext(Context) && Context.GetType() == type

	FCborReader Reader(Buffer, NumBytes);
	FCborContext Context;

	if (!Reader.ReadNext(Context) || Context.GetType() != ECborType::Map)
	{
		return;
	}

	while (Reader.ReadNext(Context))
	{
		FStringView Prop = Context.AsString();

		if (IS_PROPERTY(GStoreDirName, ECborType::String))
		{
			TS_SETTINGS_TRACE("Set -> %s to '%s'\n", GStoreDirName, *FString(Context.AsString()))
			StoreDir = fs::path(*FString(Context.AsString()));
		}
		else if (IS_PROPERTY(GAdditionalWatchDirsName, ECborType::Array))
		{
			while (Reader.ReadNext(Context) && Context.GetType() != ECborType::End )
			{
				if (Context.GetType() == ECborType::String)
				{
					FStringView Value = Context.AsString();
					if (Value.Len() > 0 && Value[0] == '-')
					{
						Value.RemovePrefix(1);
						fs::path ValuePath(*FString(Value));
						if (AdditionalWatchDirs.RemoveIf([&](const fs::path& In) { return ValuePath.compare(In) == 0; }))
						{
							TS_SETTINGS_TRACE("Removed -> %s '%s'\n", GAdditionalWatchDirsName, *FString(Value))
						}
					}
					else /*if (Value.Len() > 0)*/
					{
						FString ValueString(Value);
						
						if (AdditionalWatchDirs.FindOrAdd(fs::path(*ValueString), [](const FPath& Path, const FPath& InPath) { 
							return Path.compare(InPath); 
						}))
						{
							TS_SETTINGS_TRACE("Add -> %s '%s'\n", GAdditionalWatchDirsName, *ValueString)
						}
						else
						{
							TS_SETTINGS_TRACE("Did not add %s, existing directory with the same path was already added\n", *ValueString)
						}
					}
				}
			}
		}
		else if (IS_PROPERTY(GStorePortName, ECborType::Integer))
		{
			TS_SETTINGS_TRACE("Set -> %s to %lli\n", GStorePortName, Context.AsInteger())
			StorePort = int32(Context.AsInteger());
		}
		else if (IS_PROPERTY(GRecorderPortName, ECborType::Integer))
		{
			TS_SETTINGS_TRACE("Set -> %s to %lli\n", GRecorderPortName, Context.AsInteger())
			RecorderPort = int32(Context.AsInteger());
		}
		else if (IS_PROPERTY(GThreadCountName, ECborType::Integer))
		{
			TS_SETTINGS_TRACE("Set -> %s to %lli\n", GThreadCountName, Context.AsInteger())
			ThreadCount = int32(Context.AsInteger());
		}
		else if (IS_PROPERTY(GSponsoredName, ECborType::Integer))
		{
			TS_SETTINGS_TRACE("Set -> %s to %lli\n", GSponsoredName, Context.AsInteger())
			Sponsored = int32(Context.AsInteger());
		}
	}

	++ChangeSerial;

	#undef IS_PROPERTY
}

////////////////////////////////////////////////////////////////////////////////
void FStoreSettings::SerializeToCbor(TArray<uint8>& OutBuffer) const
{
	TInlineBuffer<1024> Buffer;
	FCborWriter Writer(Buffer);

	Writer.OpenMap();

	Writer.WriteString(GStoreDirName);
	Writer.WriteString(StoreDir.string().c_str());

	if (AdditionalWatchDirs.Num() > 0)
	{
		Writer.WriteString(GAdditionalWatchDirsName);
		Writer.OpenArray();
		for (auto& Dir : AdditionalWatchDirs)
		{
			Writer.WriteString(Dir.string().c_str());
		}
		Writer.Close();
	}

	Writer.WriteString(GStorePortName);
	Writer.WriteInteger(StorePort);

	Writer.WriteString(GRecorderPortName);
	Writer.WriteInteger(RecorderPort);

	Writer.WriteString(GThreadCountName);
	Writer.WriteInteger(ThreadCount);

	Writer.WriteString(GSponsoredName);
	Writer.WriteInteger(Sponsored);

	Writer.Close();

	OutBuffer.Append(Buffer->GetData(), int32(Buffer->GetSize()));
}

void FStoreSettings::PrintToLog() const
{
	TS_LOG("Store settings (%s):", SettingsFile.string().c_str());
	TS_LOG(" - Store port: %u", StorePort);
	TS_LOG(" - Recorder port: %u", RecorderPort);
	TS_LOG(" - Thread count: %u", ThreadCount);
	TS_LOG(" - Sponsored mode: %d", Sponsored);
	TS_LOG(" - Directory: '%s'", StoreDir.string().c_str());
	for (const auto& Dir : AdditionalWatchDirs)
	{
		TS_LOG(" - Additional watch directory: '%s'", Dir.string().c_str());
	}
}

#undef PRINTCHECKEDF
#undef PRINTCHECKED
