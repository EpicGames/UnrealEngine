// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Pch.h"
#include "Foundation.h"

class FStoreSettings
{
public:
	FPath 				SettingsFile;			// Path to settings file
	FPath				StoreDir;				// Path to active store
	TArray<FPath>		AdditionalWatchDirs;	// Additional directories to watch
	int32				StorePort		= 1989; // Changes doesn't take effect until restart
	int32				RecorderPort	= 1981; // Changes doesn't take effect until restart
	int32				ThreadCount		= 0; 	// <=0:logical CPU count
	int32				Sponsored		= 1;	// When non-zero store will close when there are
												// no more sponsor processes or connections active.

	void ReadFromSettings(const FPath& Path);
	void WriteToSettingsFile() const;
	void ApplySettingsFromCbor(const uint8* Buffer, uint32 NumBytes);
	void SerializeToCbor(TArray<uint8>& OutBuffer) const;
	uint32 GetChangeSerial() const { return ChangeSerial; }
	void PrintToLog() const;

private:
	uint32 ChangeSerial = 1;
};
