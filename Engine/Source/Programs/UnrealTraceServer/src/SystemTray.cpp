// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "StoreService.h"
#include "Version.h"

#if TS_USING(TS_PLATFORM_WINDOWS)

#define TS_WITH_SYSTEM_TRAY TS_OFF

////////////////////////////////////////////////////////////////////////////////
class FSystemTray* GSystemTray = nullptr;

////////////////////////////////////////////////////////////////////////////////
class FSystemTray
{
public:
							FSystemTray(FStoreService& InStoreService);
							~FSystemTray();

private:
	void					ThreadEntry();
	static LRESULT CALLBACK	WndProc(HWND Wnd, uint32 Msg, WPARAM wParam, LPARAM lParam);
	HWND					Wnd;
	HANDLE					Thread;
	DWORD					ThreadId;
	FStoreService&			StoreService;
};

////////////////////////////////////////////////////////////////////////////////
FSystemTray::FSystemTray(FStoreService& InStoreService)
: StoreService(InStoreService)
{
	struct FThunk
	{
		static DWORD WINAPI Entry(void* Param)
		{
			((FSystemTray*)Param)->ThreadEntry();
			return 0;
		}
	};
	Thread = CreateThread(nullptr, 0, &FThunk::Entry, this, 0, &ThreadId);
}

////////////////////////////////////////////////////////////////////////////////
FSystemTray::~FSystemTray()
{
	PostThreadMessage(ThreadId, WM_QUIT, 0, 0);
	WaitForSingleObject(Thread, 5000);
	CloseHandle(Thread);
}

////////////////////////////////////////////////////////////////////////////////
void FSystemTray::ThreadEntry()
{
	HINSTANCE Instance = GetModuleHandle(nullptr);

	HICON Icon = LoadIcon(Instance, MAKEINTRESOURCE(TS_ICON_ID));

	WNDCLASSW WndClass = {};
	WndClass.lpfnWndProc = (WNDPROC)(&FSystemTray::WndProc);
	WndClass.hInstance = Instance;
	WndClass.hIcon = Icon;
	WndClass.lpszClassName = L"UnrealTraceServer";
	RegisterClassW(&WndClass);

	Wnd = CreateWindowW(L"UnrealTraceServer", L"UnrealTraceServer", 0, 10, 10,
		493, 493, nullptr, nullptr, GetModuleHandle(nullptr), nullptr);

	NOTIFYICONDATAW NotifyIconData = {
		sizeof(NotifyIconData),
		Wnd,
		0,
		NIF_MESSAGE|NIF_ICON|NIF_TIP,
	};
	NotifyIconData.uVersion = NOTIFYICON_VERSION_4;
	NotifyIconData.uCallbackMessage = WM_APP;
	NotifyIconData.hIcon = Icon;
	wcscpy(NotifyIconData.szTip, L"UnrealTraceServer; a hub for receving and enumerating traces");
	Shell_NotifyIconW(NIM_ADD, &NotifyIconData);
	Shell_NotifyIconW(NIM_SETVERSION, &NotifyIconData);

	MSG Msg;
	while (int32(GetMessage(&Msg, nullptr, 0, 0)) > 0)
	{
		TranslateMessage(&Msg);
		DispatchMessage(&Msg);
	}

	Shell_NotifyIconW(NIM_DELETE, &NotifyIconData);
	DestroyWindow(Wnd);
}

////////////////////////////////////////////////////////////////////////////////
LRESULT CALLBACK FSystemTray::WndProc(HWND Wnd, uint32 Msg, WPARAM wParam, LPARAM lParam)
{
	if (Msg != WM_APP || (lParam != WM_LBUTTONUP && lParam != WM_RBUTTONUP))
	{
		return CallWindowProc(DefWindowProc, Wnd, Msg, wParam, lParam);
	}

	//*
	HMENU Menu = CreatePopupMenu();

	auto AddMenuItem = [Menu] (const wchar_t* Text, uint32 Id, bool bGrey)
	{
		MENUITEMINFOW MenuItemInfo = {
			sizeof(MenuItemInfo),
			MIIM_TYPE|MIIM_STATE|MIIM_ID,
			MFT_STRING,
			UINT(bGrey ? MFS_GRAYED : MFS_ENABLED),
			Id,
		};
		MenuItemInfo.dwTypeData = LPWSTR(Text);
		InsertMenuItemW(Menu, 999, TRUE, &MenuItemInfo);
	};

	AddMenuItem(L"UnrealTraceServer", 493, true);
	AddMenuItem(L"-", 494, true);
	AddMenuItem(L"Debug", 495, false);
	AddMenuItem(L"Exit", 496, false);

	POINT MousePos;
	GetCursorPos(&MousePos);
	BOOL Ret = TrackPopupMenu(
		Menu, TPM_RETURNCMD|TPM_RIGHTALIGN|TPM_BOTTOMALIGN,
		MousePos.x, MousePos.y,
		0, Wnd, nullptr);

	DestroyMenu(Menu);
	//*/

	return 0;
}



////////////////////////////////////////////////////////////////////////////////
void AddToSystemTray(FStoreService& StoreService)
{
#if TS_USING(TS_WITH_SYSTEM_TRAY)
	GSystemTray = new FSystemTray(StoreService);
#endif
}

////////////////////////////////////////////////////////////////////////////////
void RemoveFromSystemTray()
{
#if TS_USING(TS_WITH_SYSTEM_TRAY)
	delete GSystemTray;
#endif
}

#endif //TS_PLATFORM_WINDOWS

/* vim: set noexpandtab : */
