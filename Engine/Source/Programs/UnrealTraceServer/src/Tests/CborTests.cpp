// Copyright Epic Games, Inc. All Rights Reserved.
#include "../Cbor.h"
#include "../Utils.h"

////////////////////////////////////////////////////////////////////////////////
static void TestCborReader_Integer()
{
	struct
	{
		int64	Expected;
		uint32	TruthSize;
		uint8	Truth[9];
	} Cases[] = {
		{ 0,			1, { 0x00 } },
		{ 0,			2, { 0x18, 0x00 } },
		{ 0,			3, { 0x19, 0x00, 0x00 } },
		{ 0,			5, { 0x1a, 0x00, 0x00, 0x00, 0x00 } },
		{ 0,			9, { 0x1b, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00 } },
		{ 1,			1, { 0x01 } },
		{ 23,			1, { 0x17 } },
		{ 24,			2, { 0x18, 0x18 } },
		{ 255,			2, { 0x18, 0xff } },
		{ 256,			3, { 0x19, 0x01, 0x00 } },
		{ 65535,		3, { 0x19, 0xff, 0xff } },
		{ 65536,		5, { 0x1a, 0x00, 0x01, 0x00, 0x00 } },
		{ 123456,		5, { 0x1a, 0x00, 0x01, 0xe2, 0x40 } },
		{ 123456789,	5, { 0x1a, 0x07, 0x5b, 0xcd, 0x15 } },
		{ -1,			1, { 0x20 } },
		{ -2,			1, { 0x21 } },
		{ -10,			1, { 0x29 } },
		{ -24,			1, { 0x37 } },
		{ -25,			2, { 0x38, 0x18 } },
		{ -500,			3, { 0x39, 0x01, 0xf3 } },
		{ -67000,		5, { 0x3a, 0x00, 0x01, 0x05, 0xb7 } },
		{ -123456789,	5, { 0x3a, 0x07, 0x5b, 0xcd, 0x14 } },
	};

	for (const auto& Case : Cases)
	{
		FCborContext Context;
		FCborReader Reader(Case.Truth, Case.TruthSize);
		REQUIRE(Reader.ReadNext(Context));
		REQUIRE(Context.GetType() == ECborType::Integer);
		// REQUIRE(Context.GetType() == ECborType::String);
		REQUIRE(Context.AsInteger() == Case.Expected);
		REQUIRE(!Reader.ReadNext(Context));
	}
}

////////////////////////////////////////////////////////////////////////////////
static void TestCborReader_String()
{
	// "string"
	{
		const uint8 Truth[] = { 0x66, 0x73, 0x74, 0x72, 0x69, 0x6e, 0x67 };
		const char* Expected = "string";

		FCborContext Context;
		FCborReader Reader(Truth, TS_ARRAY_COUNT(Truth));
		REQUIRE(Reader.ReadNext(Context));
		REQUIRE(Context.GetType() == ECborType::String);
		REQUIRE(Context.AsString().Compare(Expected) == 0);
		REQUIRE(!Reader.ReadNext(Context));
	}

	// ""
	{
		const uint8 Truth[] = { 0x60 };
		const char* Expected = "";

		FCborContext Context;
		FCborReader Reader(Truth, TS_ARRAY_COUNT(Truth));
		REQUIRE(Reader.ReadNext(Context));
		REQUIRE(Context.GetType() == ECborType::String);
		REQUIRE(Context.AsString().Compare(Expected) == 0);
		REQUIRE(!Reader.ReadNext(Context));
	}

	// "stringstringstringstringstringstringstringstringstringstring"
	{
		const uint8 Truth[] = {
			0x78, 0x3c,
			0x73, 0x74, 0x72, 0x69, 0x6e, 0x67,
			0x73, 0x74, 0x72, 0x69, 0x6e, 0x67,
			0x73, 0x74, 0x72, 0x69, 0x6e, 0x67,
			0x73, 0x74, 0x72, 0x69, 0x6e, 0x67,
			0x73, 0x74, 0x72, 0x69, 0x6e, 0x67,
			0x73, 0x74, 0x72, 0x69, 0x6e, 0x67,
			0x73, 0x74, 0x72, 0x69, 0x6e, 0x67,
			0x73, 0x74, 0x72, 0x69, 0x6e, 0x67,
			0x73, 0x74, 0x72, 0x69, 0x6e, 0x67,
			0x73, 0x74, 0x72, 0x69, 0x6e, 0x67
		};
		const char* Expected = "stringstringstringstringstringstringstringstringstringstring";

		FCborContext Context;
		FCborReader Reader(Truth, TS_ARRAY_COUNT(Truth));
		REQUIRE(Reader.ReadNext(Context));
		REQUIRE(Context.GetType() == ECborType::String);
		REQUIRE(Context.AsString().Compare(Expected) == 0);
		REQUIRE(!Reader.ReadNext(Context));
	}
}

////////////////////////////////////////////////////////////////////////////////
static void TestCborReader_Map()
{
	// {}
	{
		const uint8 Truth[] = { 0xa0 };

		FCborContext Context;
		FCborReader Reader(Truth, TS_ARRAY_COUNT(Truth));
		REQUIRE(Reader.ReadNext(Context));
		REQUIRE(Context.GetType() == ECborType::Map);
		REQUIRE(Context.GetLength() == 0);
		REQUIRE(!Reader.ReadNext(Context));
	}

	// {...}
	{
		const uint8 Truth[] = { 0xbf, 0xff };

		FCborContext Context;
		FCborReader Reader(Truth, TS_ARRAY_COUNT(Truth));
		REQUIRE(Reader.ReadNext(Context));
		REQUIRE(Context.GetType() == ECborType::Map);
		REQUIRE(Context.GetLength() == -1);

		REQUIRE(!Reader.ReadNext(Context));
		REQUIRE(Context.GetType() == ECborType::End);

		REQUIRE(!Reader.ReadNext(Context));
		REQUIRE(Context.GetType() == ECborType::Eof);
	}

	// { "key":"value", 0:1 }
	{
		const uint8 Truth[] = {
			0xa2,
			0x63, 0x6b, 0x65, 0x79,
			0x65, 0x76, 0x61, 0x6c, 0x75, 0x65,
			0x00, 0x01
		};

		FCborContext Context;
		FCborReader Reader(Truth, TS_ARRAY_COUNT(Truth));

		REQUIRE(Reader.ReadNext(Context));
		REQUIRE(Context.GetType() == ECborType::Map);
		REQUIRE(Context.GetLength() == 2);

		REQUIRE(Reader.ReadNext(Context));
		REQUIRE(Context.GetType() == ECborType::String);
		REQUIRE(Context.AsString().Compare("key") == 0);

		REQUIRE(Reader.ReadNext(Context));
		REQUIRE(Context.GetType() == ECborType::String);
		REQUIRE(Context.AsString().Compare("value") == 0);

		REQUIRE(Reader.ReadNext(Context));
		REQUIRE(Context.GetType() == ECborType::Integer);
		REQUIRE(Context.AsInteger() == 0);

		REQUIRE(Reader.ReadNext(Context));
		REQUIRE(Context.GetType() == ECborType::Integer);
		REQUIRE(Context.AsInteger() == 1);

		REQUIRE(!Reader.ReadNext(Context));
		REQUIRE(Context.GetType() == ECborType::Eof);
	}
}

// {{{1 test-write -------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
static void TestCborWriter_Integer()
{
	struct
	{
		int64	Content;
		uint32	TruthSize;
		uint8	Truth[9];
	} Cases[] = {
		{ 0,			1, { 0x00 } },
		{ 1,			1, { 0x01 } },
		{ 23,			1, { 0x17 } },
		{ 24,			2, { 0x18, 0x18 } },
		{ 255,			2, { 0x18, 0xff } },
		{ 256,			3, { 0x19, 0x01, 0x00 } },
		{ 65535,		3, { 0x19, 0xff, 0xff } },
		{ 65536,		5, { 0x1a, 0x00, 0x01, 0x00, 0x00 } },
		{ 123456,		5, { 0x1a, 0x00, 0x01, 0xe2, 0x40 } },
		{ 123456789,	5, { 0x1a, 0x07, 0x5b, 0xcd, 0x15 } },
		{ -1,			1, { 0x20 } },
		{ -2,			1, { 0x21 } },
		{ -10,			1, { 0x29 } },
		{ -24,			1, { 0x37 } },
		{ -25,			2, { 0x38, 0x18 } },
		{ -500,			3, { 0x39, 0x01, 0xf3 } },
		{ -67000,		5, { 0x3a, 0x00, 0x01, 0x05, 0xb7 } },
		{ -123456789,	5, { 0x3a, 0x07, 0x5b, 0xcd, 0x14 } },
	};

	for (const auto& Case : Cases)
	{
		TInlineBuffer<4> Buffer;
		FCborWriter Writer(Buffer);
		Writer.WriteInteger(Case.Content);
		REQUIRE(Buffer->GetSize() == Case.TruthSize);
		REQUIRE(memcmp(Buffer->GetData(), Case.Truth, Case.TruthSize) == 0);
	}
}

////////////////////////////////////////////////////////////////////////////////
static void TestCborWriter_String()
{
	struct
	{
		const char*	Value;
		uint32		TruthSize;
		const uint8	Truth[64];
	} Cases[] = {
		{ "string", 7, { 0x66, 0x73, 0x74, 0x72, 0x69, 0x6e, 0x67 } },
		{ "",		1, { 0x60 } },
		{
			"stringstringstringstringstringstringstringstringstringstring",
			62,
			{ 0x78, 0x3c,
			  0x73, 0x74, 0x72, 0x69, 0x6e, 0x67, 0x73, 0x74, 0x72, 0x69, 0x6e, 0x67,
			  0x73, 0x74, 0x72, 0x69, 0x6e, 0x67, 0x73, 0x74, 0x72, 0x69, 0x6e, 0x67,
			  0x73, 0x74, 0x72, 0x69, 0x6e, 0x67, 0x73, 0x74, 0x72, 0x69, 0x6e, 0x67,
			  0x73, 0x74, 0x72, 0x69, 0x6e, 0x67, 0x73, 0x74, 0x72, 0x69, 0x6e, 0x67,
			  0x73, 0x74, 0x72, 0x69, 0x6e, 0x67, 0x73, 0x74, 0x72, 0x69, 0x6e, 0x67 },
		},
	};

	for (const auto& Case : Cases)
	{
		TInlineBuffer<8> Buffer;
		FCborWriter Writer(Buffer);
		Writer.WriteString(Case.Value);
		REQUIRE(Buffer->GetSize() == Case.TruthSize);
		REQUIRE(memcmp(Buffer->GetData(), Case.Truth, Case.TruthSize) == 0);
	}
}

////////////////////////////////////////////////////////////////////////////////
void TestCborWriter_Map()
{
	// {}
	{
		const uint8 Truth[] = { 0xa0 };

		TInlineBuffer<4> Buffer;
		FCborWriter Writer(Buffer);
		Writer.OpenMap(0);

		REQUIRE(memcmp(Buffer->GetData(), Truth, sizeof(Truth)) == 0);
	}

	// {...}
	{
		const uint8 Truth[] = { 0xbf, 0xff };

		TInlineBuffer<4> Buffer;
		FCborWriter Writer(Buffer);
		Writer.OpenMap();
		Writer.Close();

		REQUIRE(memcmp(Buffer->GetData(), Truth, sizeof(Truth)) == 0);
	}

	// { "key":"value", 0:1 }
	{
		const uint8 Truth[] = {
			0xa2,
			0x63, 0x6b, 0x65, 0x79,
			0x65, 0x76, 0x61, 0x6c, 0x75, 0x65,
			0x00, 0x01
		};

		TInlineBuffer<4> Buffer;
		FCborWriter Writer(Buffer);
		Writer.OpenMap(2);
		Writer.WriteString("key");
		Writer.WriteString("value", 5);
		Writer.WriteInteger(0);
		Writer.WriteInteger(1);

		REQUIRE(memcmp(Buffer->GetData(), Truth, sizeof(Truth)) == 0);
	}
}



// {{{1 test -------------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
void TestCbor()
{
	TestCborReader_Integer();
	TestCborReader_String();
	TestCborReader_Map();

	TestCborWriter_Integer();
	TestCborWriter_String();
	TestCborWriter_Map();

	// [-1]
	//const uint8 Truth[] = { 0x81, 0x20 };

	// float
	//const uint8 Truth[] = { 0xfb 0x3f 0xb9 0x99 0x99 0x99 0x99 0x99 0x9a }; // primitive(4591870180066957722)
}


int main(int ArgC, char** ArgV)
{
  TestCbor();
  return 0;
}
