// Copyright Epic Games, Inc. All Rights Reserved.

#include "Pch.h"
#include "AsioSocket.h"
#include "Recorder.h"
#include "TraceRelay.h"

////////////////////////////////////////////////////////////////////////////////
FTraceRelay::FTraceRelay(
	asio::io_context& IoContext,
	FAsioReadable* InInput,
	uint32 InSessionId,
	FRecorder& InRecorder)
: FAsioTcpServer(IoContext)
, FAsioTickable(IoContext)
, Input(InInput)
, Recorder(InRecorder)
, SessionId(InSessionId)
{
	StartServer();
	TickOnce(3000);
}

////////////////////////////////////////////////////////////////////////////////
FTraceRelay::~FTraceRelay()
{
	check(!Input->IsOpen());
	delete Input;

	if (Output != nullptr)
	{
		check(!Output->IsOpen());
		delete Output;
	}
}

////////////////////////////////////////////////////////////////////////////////
void FTraceRelay::Close()
{
	if (Output != nullptr)
	{
		Output->Close();
	}

	Input->Close();

	FAsioTcpServer::Close();
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceRelay::IsOpen() const
{
	if (FAsioTcpServer::IsOpen())
	{
		return true;
	}

	bool bOpen = Input->IsOpen();
	if (Output != nullptr)
	{
		bOpen &= Output->IsOpen();
	}

	return bOpen;
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceRelay::OnAccept(asio::ip::tcp::socket& Socket)
{
	Output = new FAsioSocket(Socket);
	StopTick();
	OnIoComplete(OpStart, 0);
	return false;
}

////////////////////////////////////////////////////////////////////////////////
void FTraceRelay::OnTick()
{
	if (FAsioTcpServer::IsOpen())
	{
		Close();
		return;
	}

	return OnIoComplete(OpStart, 0);
}

////////////////////////////////////////////////////////////////////////////////
void FTraceRelay::OnIoComplete(uint32 Id, int32 Size)
{
	if (Size < 0)
	{
		if (Id == OpRead && SessionId && -Size == asio::error::eof)
		{
			for (int i = 0, n = Recorder.GetSessionCount(); i < n; ++i)
			{
				const FRecorder::FSession* Session = Recorder.GetSessionInfo(i);
				if (Session->GetId() == SessionId)
				{
					TickOnce(200);
					return;
				}
			}
		}

		Output->Close();
		Input->Close();
		return;
	}

	switch (Id)
	{
	case OpStart:
	case OpSend:
		Input->Read(Buffer, BufferSize, this, OpRead);
		break;

	case OpRead:
		Output->Write(Buffer, Size, this, OpSend);
		break;
	}
}

/* vim: set noexpandtab : */
