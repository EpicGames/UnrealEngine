// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Asio.h"
#include "AsioIoable.h"
#include "AsioTcpServer.h"
#include "AsioTickable.h"

class FRecorder;
class FAsioSocket;
class FAsioReadable;

////////////////////////////////////////////////////////////////////////////////
class FTraceRelay
	: public FAsioTcpServer
	, public FAsioIoSink
	, public FAsioTickable
{
public:
						FTraceRelay(asio::io_context& IoContext, FAsioReadable* InInput, uint32 InTraceIf, FRecorder& InRecorder);
						~FTraceRelay();
	bool				IsOpen() const;
	void				Close();

private:
	virtual bool		OnAccept(asio::ip::tcp::socket& Socket) override;
	virtual void		OnTick() override;
	virtual void		OnIoComplete(uint32 Id, int32 Size) override;
	enum				{ OpStart, OpRead, OpSend };
	static const uint32	BufferSize = 64 << 10;
	FAsioReadable*		Input;
	FAsioSocket*		Output = nullptr;
	FRecorder&			Recorder;
	uint32				SessionId;
	uint8				Buffer[BufferSize];
};

/* vim: set noexpandtab : */
