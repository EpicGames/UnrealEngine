// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Foundation.h"

////////////////////////////////////////////////////////////////////////////////
class FInlineBufferBase
{
public:
				FInlineBufferBase() = default;
				FInlineBufferBase(uint32 InlineMax);
	uint32		GetSize() const;
	bool		HasOverflowed() const;

protected:
	void		SetMax(uint32 NewMax);
	uint32		GetMax() const;
	uint32		Used = 0;
	int32		Max;
};

////////////////////////////////////////////////////////////////////////////////
inline FInlineBufferBase::FInlineBufferBase(uint32 InlineMax)
: Max(int32(~InlineMax))
{
}

////////////////////////////////////////////////////////////////////////////////
inline uint32 FInlineBufferBase::GetSize() const
{
	return Used;
}

////////////////////////////////////////////////////////////////////////////////
inline bool FInlineBufferBase::HasOverflowed() const
{
	return Max > 0;
}

////////////////////////////////////////////////////////////////////////////////
inline void FInlineBufferBase::SetMax(uint32 NewMax)
{
	Max = NewMax;
}

////////////////////////////////////////////////////////////////////////////////
inline uint32 FInlineBufferBase::GetMax() const
{
	return HasOverflowed() ? Max : ~Max;
}



////////////////////////////////////////////////////////////////////////////////
class FInlineBuffer
	: public FInlineBufferBase
{
public:
	uint8*			Append(uint32 AppendSize);
	void			Append(const void* Data, uint32 DataSize);
	const uint8*	GetData() const;

private:
	uint8			Buffer[];
};

////////////////////////////////////////////////////////////////////////////////
inline uint8* FInlineBuffer::Append(uint32 AppendSize)
{
	auto GrowthPolicy = [] (uint32 Size, uint32 Demand) -> uint32 {
		uint32 NewSize = Size + 256;
		if (Demand > NewSize)
		{
			NewSize = Demand + 256;
		}
		return NewSize;
	};

	auto** Ptr = (uint8**)Buffer;
	uint32 CurrentMax = GetMax();
	uint32 NextUsed = Used + AppendSize;
	if (CurrentMax < NextUsed)
	{
		uint32 NewMax = GrowthPolicy(CurrentMax, NextUsed);
		if (HasOverflowed())
		{
			Ptr[0] = (uint8*)realloc(Ptr[0], NewMax);
		}
		else
		{
			uint8* NewBuffer = (uint8*)malloc(NewMax);
			memcpy(NewBuffer, Buffer, Used);
			Ptr[0] = NewBuffer;
		}
		SetMax(NewMax);
	}

	uint8* Cursor = HasOverflowed() ? Ptr[0] : Buffer;
	Cursor += Used;
	Used = NextUsed;
	return Cursor;
}

////////////////////////////////////////////////////////////////////////////////
inline void FInlineBuffer::Append(const void* Data, uint32 DataSize)
{
	uint8* Cursor = Append(DataSize);
	memcpy(Cursor, Data, DataSize);
}

////////////////////////////////////////////////////////////////////////////////
inline const uint8* FInlineBuffer::GetData() const
{
	return HasOverflowed() ? *(uint8**)Buffer : Buffer;
}



////////////////////////////////////////////////////////////////////////////////
template <int Size>
class TInlineBuffer
	: protected FInlineBufferBase
{
public:
					TInlineBuffer();
					~TInlineBuffer();
					operator FInlineBuffer& ();
	FInlineBuffer*	operator -> ();

private:
	enum { BufferSize = (Size >= 16) ? Size : 16 };

	union
	{
		uint8*		Ptr;
		uint8		Buffer[BufferSize];
	};
};

////////////////////////////////////////////////////////////////////////////////
template <int Size>
TInlineBuffer<Size>::TInlineBuffer()
: FInlineBufferBase(BufferSize)
{
}

////////////////////////////////////////////////////////////////////////////////
template <int Size>
TInlineBuffer<Size>::~TInlineBuffer()
{
	if (HasOverflowed())
	{
		delete Ptr;
	}
}

////////////////////////////////////////////////////////////////////////////////
template <int Size>
TInlineBuffer<Size>::operator FInlineBuffer& ()
{
	return *(FInlineBuffer*)this;
}

////////////////////////////////////////////////////////////////////////////////
template <int Size>
FInlineBuffer* TInlineBuffer<Size>::operator -> ()
{
	return (FInlineBuffer*)this;
}



////////////////////////////////////////////////////////////////////////////////
template <typename Type>
inline uint32 QuickStoreHash(const Type* String)
{
	uint32 Value = 5381;
	for (; *String; ++String)
	{
		Value = ((Value << 5) + Value) + *String;
	}
	return Value;
}

////////////////////////////////////////////////////////////////////////////////
inline uint32 QuickStoreHash(FStringView View)
{
	const char* String = View.GetData();
	uint32 Value = 5381;
	for (int i = View.Len(); i > 0; --i, ++String)
	{
		Value = ((Value << 5) + Value) + *String;
	}
	return Value;
}

////////////////////////////////////////////////////////////////////////////////

#if defined(_MSC_VER)
#   define DEBUG_BREAK() do { __debugbreak(); } while (0)
#else
#   define DEBUG_BREAK() do { __builtin_trap(); exit(-1); } while (0)
#endif

#define REQUIRE(x) \
	do { \
		if (!(x)) { \
			fprintf(stderr, "Test failed %s:%u (%s): \"%s\"", __FILE__, __LINE__, __func__, #x); \
			DEBUG_BREAK(); \
		} \
	} while (false)



/* vim: set noexpandtab : */
