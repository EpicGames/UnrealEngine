// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !defined(TS_VERSION_PROTOCOL)
#	define TS_VERSION_PROTOCOL	1
#endif

#if !defined(TS_VERSION_MINOR)
#	define TS_VERSION_MINOR		24
#endif

#define TS_ICON_ID				493

/* vim: set noexpandtab : */
