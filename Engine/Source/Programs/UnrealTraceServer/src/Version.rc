// Copyright Epic Games, Inc. All Rights Reserved.

#include "Version.h"

1 VERSIONINFO
	FILEVERSION		TS_VERSION_PROTOCOL, TS_VERSION_MINOR
	PRODUCTVERSION	5 /* UE5 */
{
	BLOCK "StringFileInfo" {
		BLOCK "040904b0" {
			VALUE "CompanyName",	"Epic Games"
			VALUE "ProductName",	"Unreal Engine Trace Server"
			VALUE "FileDescription","Hub for UE5 runtime traces (i.e. -tracehost=<host>) and tools like Unreal Insights"
			VALUE "ProductVersion",	"UE5"
		}
	}
	BLOCK "VarFileInfo" {
		VALUE "Translation", 0x0409, 1200
	}
}

TS_ICON_ID ICON "icon.ico"

/* vim: set noexpandtab : */
