-- Copyright Epic Games, Inc. All Rights Reserved.

add_rules("mode.debug", "mode.release")

-- Musl option ------------------------------------------------------
-- In order to produce a distribution independent binary on Linux
-- we statically link with musl to avoid using the system libc/libc++
-- libraries.

option("musl")
	-- Not needed but useful for debugging
	add_defines('TS_WITH_MUSL')
option_end()

if has_config('musl') then
	add_requires("muslcc")
end

-- Third party libs -------------------------------------------------

ASIO_VER = "1.12.2"
add_includedirs("thirdparty/asio/"..ASIO_VER)

add_includedirs("thirdparty/cxxopts/include")
add_defines("CXXOPTS_NO_EXCEPTIONS")

-- Configuration options --------------------------------------------

-- Enable debug symbols for all configurations
set_symbols("debug")

if is_mode("debug") then
	add_defines("TS_BUILD_DEBUG=TS_ON")
end

if is_mode("release") then
	set_optimize("fastest")
end

-- Platform options -------------------------------------------------
-- Todo: This doesn't work if we ever want to cross compile. Need
-- figure out the toolchain rather than target platform

if is_plat("linux") then
	add_syslinks("pthread", "rt")
end

if is_plat("windows") then
	set_runtimes("MT")
	add_cxflags("/nologo")
	add_ldflags("/nologo",  "/subsystem:console", "/incremental:no", "/opt:ref,icf")
	add_syslinks("dbghelp")
end

if is_plat("macosx") then
	add_frameworks("CoreServices", "CoreFoundation")
	add_ldflags("-lpthread")
end



-- Main server target -----------------------------------------------

target("UnrealTraceServer")
	set_kind("binary")
	set_languages("c++17")
	set_warnings("all")
	set_exceptions("no-cxx")
	set_pcxxheader("src/Pch.h")
	add_files("src/*.cpp")
	-- Add header files only for project generation
	add_headerfiles("src/*.h", {install=false})
	if is_plat("windows") then
		add_files("src/*.rc")
	end
	-- Todo: If we use the package manager in xmake
	--add_packages('cxxopts', 'asio')
	add_options("musl")
	if has_config('musl') then
		add_ldflags('-static')
		set_toolchains("@muslcc")
	end
	on_install(function(target)
			if is_mode("release") then
				import("core.project.task")
				task.run("deploy_target", {}, target)
			else
				cprint("${red} Must be in 'release' mode to install.${clear}")
				os.exit(1)
			end
		end
	)


-- Tests ------------------------------------------------------------
-- This builds one target file for each file in src/Tests and is 
-- designed to be used with Xmake built in test runner. Each test file
-- is expected to have a separate main entry point.
for _, file in ipairs(os.files("src/Tests/*.cpp")) do
     local name = path.basename(file)
     target(name)
        set_kind("binary")
        set_default(false)
        add_files("src/Tests/" .. name .. ".cpp")
        add_tests("default")
        set_languages("c++17")
        set_warnings("all")
        set_exceptions("no-cxx")
        set_pcxxheader("src/Pch.h")
        add_files("src/*.cpp")
        remove_files("src/Main.cpp")
        -- Add header files only for project generation
        add_headerfiles("src/*.h", {install=false})
        if is_plat("windows") then
          add_files("src/*.rc")
        end
        -- Todo: If we use the package manager in xmake
        --add_packages('cxxopts', 'asio')
        add_options("musl")
        if has_config('musl') then
          add_ldflags('-static')
          set_toolchains("@muslcc")
        end
end


-- Deploy task ------------------------------------------------------

task("deploy_target")
	on_run("deploy")

-- Version task ------------------------------------------------------
task("bundle")
  set_menu {
      usage = "xmake bundle",
      description = "Create Zip bundle from binaries",
      options = {
        {nil, "codesignidentity", "v", nil, "Code signing identity"},
      }
  }
  on_run(function ()
      import("scripts.bundle")
      bundle()
  end) 

-- Version task ------------------------------------------------------
task("sourceversion")
  set_menu {
    usage = "xmake sourceversion",
    description = "Prints current version defined in source."
  }
  on_run(
    function()
      import("scripts.version")
      print(version())
    end
  )
